// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubobjectData.h"
#include "Engine/Blueprint.h"			// Casting to UBlueprint
#include "Engine/BlueprintGeneratedClass.h"
#include "Components/ChildActorComponent.h"
#include "GameFramework/Actor.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ComponentAssetBroker.h"			// FComponentAssetBrokerage
#include "SubobjectDataSubsystem.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/World.h"					// Finding the instance of a component on an actor
#include "Engine/InheritableComponentHandler.h"
#include "Engine/SCS_Node.h"		// #TODO_BH  We need to remove this when the actual subobject refactor happens

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubobjectData)

#define LOCTEXT_NAMESPACE "SubobjectData"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
#define REDIRECT_TO_ADAPTER(FunctionName, ...) \
	if (const FSubobjectDataAdapterBase* Adapter = USubobjectDataSubsystem::Get()->FindDataAdapter(*this))				\
	{																													\
		return Adapter->FunctionName(*this, ##__VA_ARGS__);																\
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


FSubobjectData::FSubobjectData()
	: WeakObjectPtr(nullptr)
	, ParentObjectHandle(FSubobjectDataHandle::InvalidHandle)
	, SCSNodePtr(nullptr)
{
	// By Default the handle will be invalid, it will only
	// be generated if we are given an object.
}

FSubobjectData::FSubobjectData(UObject* ContextObject, const FSubobjectDataHandle& InParentHandle, const bool bInIsInheritedSCS)
	: ParentObjectHandle(InParentHandle)
{
	Initialize(ContextObject, InParentHandle, bInIsInheritedSCS);
}

void FSubobjectData::Initialize(UObject* ContextObject, const FSubobjectDataHandle& InParentHandle, const bool bInIsInheritedSCS)
{
	WeakObjectPtr = ContextObject;
	ParentObjectHandle = InParentHandle.GetSharedDataPtr();
	SCSNodePtr = nullptr;
	
	FSubobjectDataSubsystemContextData_SingleUObjectContextObject SingleUObject(ContextObject);
	ContextData.InitializeAsScriptStruct(FSubobjectDataSubsystemContextData_SingleUObjectContextObject::StaticStruct(), reinterpret_cast<const uint8*>(&SingleUObject));
	
	// Check for a BP added CAC
	if (USCS_Node* SCS = Cast<USCS_Node>(ContextObject))
	{
		if (SCS->ComponentTemplate && SCS->ComponentTemplate->IsA<UChildActorComponent>())
		{
			bIsChildActor = true;
		}
	}
	else if (ContextObject && ContextObject->IsA<UChildActorComponent>())
	{
		bIsChildActor = true;
	}

	if (UActorComponent* Component = Cast<UActorComponent>(ContextObject))
	{
		// Create an inherited subobject data
		if (bInIsInheritedSCS || Component->CreationMethod != EComponentCreationMethod::Instance)
		{
			bIsInheritedSubobject = true;
		}
	}
	else if (USCS_Node* SCS = Cast<USCS_Node>(ContextObject))
	{
		if (UActorComponent* Comp = SCS->ComponentTemplate)
		{
			bIsInheritedSubobject = (Comp->CreationMethod != EComponentCreationMethod::Instance);
		}
	}

	bIsInheritedSCS = bInIsInheritedSCS && bIsInheritedSubobject;

	AttemptToSetSCSNode();
}

void FSubobjectData::Initialize(TInstancedStruct<FSubobjectDataSubsystemContextDataBase> InContextData, const FSubobjectDataHandle& ParentHandle)
{
	// Avoid double initializations
	if (ContextData.IsValid())
	{
		return;
	}

	ContextData = MoveTemp(InContextData);
	ParentObjectHandle = ParentHandle;
	if (ParentHandle.IsValid())
	{
		ParentHandle.GetData()->AddChildHandleOnly(GetHandle());
	}
}

bool FSubobjectData::CanEdit() const
{
	REDIRECT_TO_ADAPTER(CanEdit);
	
	if (bIsInheritedSubobject)
	{
		if (IsComponent())
		{
			if (IsInstancedInheritedComponent())
			{
				const UActorComponent* Template = GetComponentTemplate();
				return (Template ? Template->IsEditableWhenInherited() : false);
			}
			else if (!IsNativeComponent())
			{
				USCS_Node* SCS_Node = GetSCSNode();
				return (SCS_Node != nullptr);
			}
			else if (const UActorComponent* ComponentTemplate = GetComponentTemplate())
			{
				return FComponentEditorUtils::GetPropertyForEditableNativeComponent(ComponentTemplate) != nullptr;
			}
		}
	}

	// Actors are editable by default
	if(const AActor* ActorContext = GetObject<AActor>())
	{
		return true;
	}
	// If this is an instance-added component, then we can edit it. We know this isn't inherted because it would
	// be a FInheritedSubobjectData
	else if(const UActorComponent* Component = GetComponentTemplate())
	{
		if(Component->CreationMethod == EComponentCreationMethod::Instance)
		{
			return true;
		}
	}
	return false;
}

bool FSubobjectData::CanDelete() const
{
	REDIRECT_TO_ADAPTER(CanDelete);

	if (bIsInheritedSubobject)
	{
		if (IsInheritedComponent() || (IsDefaultSceneRoot() && SceneRootHasDefaultName()) || (GetSCSNode() != nullptr && IsInstancedInheritedComponent()) || IsChildActorSubtreeObject())
		{
			return false;
		}

		return true;
	}

	// Components can be deleted if they are not inherited
	if (const UActorComponent* ComponentTemplate = GetComponentTemplate())
	{
		// Inherited components cannot be deleted
		if (IsInheritedComponent() || IsChildActor())
		{
			return false;
		}
		// You can delete the default scene root on instances of Native C++ actors in the level
		// but not inside of a blueprint or on BP created actors
		else if (IsDefaultSceneRoot())
		{
			if (const AActor* Actor = ComponentTemplate->GetOwner())
			{
				return Actor->GetClass()->IsNative();
			}

			// Otherwise you can't delete the default scene root
			return false;
		}
		else
		{
			// You can delete any instance-added component
			return IsInstancedComponent();
		}
	}
	
	// Otherwise, it can't be deleted
	return false;
}

bool FSubobjectData::CanDuplicate() const
{
	REDIRECT_TO_ADAPTER(CanDuplicate);

	return CanCopy();
}

bool FSubobjectData::CanCopy() const
{
	REDIRECT_TO_ADAPTER(CanCopy);
	
	if (IsInstancedInheritedComponent())
	{
		return FComponentEditorUtils::CanCopyComponent(GetComponentTemplate()) && !SceneRootHasDefaultName();
	}
	
	return FComponentEditorUtils::CanCopyComponent(GetComponentTemplate());
}

bool FSubobjectData::CanReparent() const
{
	REDIRECT_TO_ADAPTER(CanReparent);
	
	if(IsComponent())
	{
		if(GetSCSNode() != nullptr)
		{
			return !IsInstancedInheritedComponent() && !IsInheritedComponent();
		}
		
		return !IsInheritedComponent() && !IsDefaultSceneRoot() && IsSceneComponent();
	}
	
	return false;
}

bool FSubobjectData::CanRename() const
{
	REDIRECT_TO_ADAPTER(CanRename);
	
	if(!IsValid())
	{
		return false;
	}
	
	// You can rename instance actors in the level editor, but you cannot rename
	// the default actors in blueprints. 
	if(IsActor())
	{
		return IsInstancedActor() && !IsChildActor();
	}

	// You can rename within the BP editor, but not if if this subobject
	// is on an instance in the level
	if(GetSCSNode() != nullptr && !IsInheritedSCSNode())
	{
		return !IsInstancedInheritedComponent() && !SceneRootHasDefaultName();
	}
	
	return !IsInheritedComponent() && !IsDefaultSceneRoot() && !IsChildActorSubtreeObject();
}

const UObject* FSubobjectData::GetObjectForBlueprint(UBlueprint* Blueprint) const
{
	REDIRECT_TO_ADAPTER(GetObjectForBlueprint, Blueprint);
	
	const bool bCanEdit = CanEdit();
	// We have to deal with the ICH (Inherited Component Handler) for components 
	if(IsComponent() && bCanEdit && !IsNativeComponent() && IsInheritedSCSNode())
	{
		UActorComponent* OverriddenComponent = nullptr;
		FComponentKey Key(GetSCSNode());
		
		const bool bBlueprintCanOverrideComponentFromKey = Key.IsValid()
		    && Blueprint
		    && Blueprint->ParentClass
		    && Blueprint->ParentClass->IsChildOf(Key.GetComponentOwner());
		
		if (bBlueprintCanOverrideComponentFromKey)
		{
			const bool bCreateIfNecessary = true;
			UInheritableComponentHandler* InheritableComponentHandler = Blueprint->GetInheritableComponentHandler(bCreateIfNecessary);
			if (InheritableComponentHandler)
			{
				OverriddenComponent = InheritableComponentHandler->GetOverridenComponentTemplate(Key);
				if (!OverriddenComponent && bCreateIfNecessary)
				{
					OverriddenComponent = InheritableComponentHandler->CreateOverridenComponentTemplate(Key);
				}
			}
		}
		return OverriddenComponent;
	}
	
	// If this not a component, then we can simply return the current object as long as it's editable
	return bCanEdit ? WeakObjectPtr.Get() : nullptr;
}

bool FSubobjectData::IsInstancedComponent() const
{
	REDIRECT_TO_ADAPTER(IsInstancedComponent);
	
	// Check the flags on the component (RF_ClassDefaultObject | RF_ArchetypeObject)
	const UActorComponent* Component = GetComponentTemplate();
	return Component && !Component->IsTemplate();
}

UActorComponent* FSubobjectData::FindMutableComponentInstanceInActor(const AActor* InActor) const
{
	REDIRECT_TO_ADAPTER(FindMutableComponentInstanceInActor, InActor);
	
	const USCS_Node* SCS_Node = GetSCSNode();
	const UActorComponent* ComponentTemplate = GetComponentTemplate();

	UActorComponent* ComponentInstance = nullptr;
	if (InActor)
	{
		if (SCS_Node)
		{
			FName VariableName = SCS_Node->GetVariableName();
			if (VariableName != NAME_None)
			{
				UWorld* World = InActor->GetWorld();
				FObjectPropertyBase* Property = FindFProperty<FObjectPropertyBase>(InActor->GetClass(), VariableName);
				if (Property != nullptr)
				{
					// Return the component instance that's stored in the property with the given variable name
					ComponentInstance = Cast<UActorComponent>(Property->GetObjectPropertyValue_InContainer(InActor));
				}
				else if (World != nullptr && World->WorldType == EWorldType::EditorPreview)
				{
					// If this is the preview actor, return the cached component instance that's being used for the preview actor prior to recompiling the Blueprint
					ComponentInstance = SCS_Node->EditorComponentInstance.Get();
				}
			}
		}
		else if (ComponentTemplate)
		{
			TInlineComponentArray<UActorComponent*> Components;
			InActor->GetComponents(Components);
			ComponentInstance = FComponentEditorUtils::FindMatchingComponent(ComponentTemplate, Components); 
		}

		if (!ComponentInstance && SCS_Node)
		{
			TInlineComponentArray<UActorComponent*> Components;
			InActor->GetComponents(Components);

			UActorComponent** MatchingArchetype = Components.FindByPredicate([SCS_Node](const UActorComponent* A)
			{
				return A && A->GetArchetype() == SCS_Node->ComponentTemplate;
			});

			ComponentInstance = MatchingArchetype ? *MatchingArchetype : nullptr;
		}
	}

	return ComponentInstance;
}

UBlueprint* FSubobjectData::GetBlueprint() const
{
	REDIRECT_TO_ADAPTER(GetBlueprint);
	
	// If this object is a BP, we can just return that
	if (UBlueprint* BP = Cast<UBlueprint>(WeakObjectPtr.Get()))
	{
		return BP;
	}
	// If it is an actor, then we can get the BP from the UClass
	else if (const AActor* DefaultActor = GetObject<AActor>())
	{
		return UBlueprint::GetBlueprintFromClass(DefaultActor->GetClass());
	}
	// For components, we need to get the blueprint from their owning actor or the SCS
	else if(IsComponent())
	{
		if (const USCS_Node* SCS_Node = GetSCSNode())
		{
			if (const USimpleConstructionScript* SCS = SCS_Node->GetSCS())
			{
				return SCS->GetBlueprint();
			}
		}
		else if(const UActorComponent* ActorComponent = GetComponentTemplate())
		{
			if (const AActor* Actor = ActorComponent->GetOwner())
			{
				return UBlueprint::GetBlueprintFromClass(Actor->GetClass());
			}
		}
	}

	return nullptr;
}

UBlueprint* FSubobjectData::GetBlueprintBeingEdited() const
{
	REDIRECT_TO_ADAPTER(GetBlueprintBeingEdited);
	
	UObject* Object = WeakObjectPtr.Get();
	if (!Object)
	{
		return nullptr;
	}

	if (UBlueprint* BP = Cast<UBlueprint>(Object))
	{
		return BP;
	}
	else
	{
		// walk to the root handle and see if it's a CDO or blueprint root object:
		FSubobjectDataHandle RootHandle = GetRootSubobject();
		if (const FSubobjectData* RootData = RootHandle.GetData())
		{
			UObject* Obj = RootData->WeakObjectPtr.Get();
			if(Obj && Obj->HasAnyFlags(RF_ClassDefaultObject))
			{
				return Cast<UBlueprint>(Obj->GetClass()->ClassGeneratedBy);
			}
			else if(const UBlueprintGeneratedClass* BPGC = Cast<UBlueprintGeneratedClass>(Obj))
			{
				return Cast<UBlueprint>(BPGC->ClassGeneratedBy);
			}
			else
			{
				return Cast<UBlueprint>(Obj);
			}
		}
	}

	return nullptr;
}

FString FSubobjectData::GetDisplayString(bool bShowNativeComponentNames /* = true */) const
{
	REDIRECT_TO_ADAPTER(GetDisplayString, bShowNativeComponentNames);
	
	FName VariableName = GetVariableName();

	const UActorComponent* ComponentTemplate = GetComponentTemplate();
	
	UBlueprint* Blueprint = GetBlueprint();
	UClass* VariableOwner = (Blueprint != nullptr) ? *Blueprint->SkeletonGeneratedClass : nullptr;
	FProperty* VariableProperty = FindFProperty<FProperty>(VariableOwner, VariableName);

	bool const bHasValidVarName = (VariableName != NAME_None);
	bool const bIsArrayVariable = bHasValidVarName && (VariableOwner != nullptr) &&
		VariableProperty && VariableProperty->IsA<FArrayProperty>();

	// Only display SCS node variable names in the tree if they have not been autogenerated
	if (ComponentTemplate && bHasValidVarName && !bIsArrayVariable)
	{
		const bool bIsNative = IsNativeComponent();
		const bool bIsInherited = IsInheritedComponent();
		const bool bIsInstance = IsInstancedComponent();
		const bool bIsBlueprintInstanceInherited = bIsInherited && bIsInstance;
		
		// Inherited/Native components will have "Name (Inherited)" as their display
		if ((bIsNative || bIsInherited) && bShowNativeComponentNames)
		{
			FStringFormatNamedArguments Args;
			Args.Add(TEXT("VarName"), VariableProperty && VariableProperty->IsNative() ? VariableProperty->GetDisplayNameText().ToString() : VariableName.ToString());
			FString CompName = TEXT(" (") + ComponentTemplate->GetName() + TEXT(")");
			Args.Add(TEXT("CompName"), bIsNative ? CompName : TEXT(""));
			return FString::Format(TEXT("{VarName}{CompName}"), Args);
		}
		else
		{
			return VariableName.ToString();
		}
	}
	else if (ComponentTemplate != nullptr)
	{
		return ComponentTemplate->GetFName().ToString();
	}
	else if (const AActor* DefaultActor = GetObject<AActor>())
	{
		FString Name;
		if (Blueprint != nullptr && !IsInstancedActor())
		{
			Blueprint->GetName(Name);
		}
		else
		{
			Name = DefaultActor->GetActorLabel();
		}
		
		FStringFormatNamedArguments Args;
		Args.Add(TEXT("ActorName"), Name);
		return FString::Format(TEXT("{ActorName}"), Args);
	}
	// If the context is a simple UObject, then we can get it's name
	else if (const UObject* Context = GetObject())
	{
		FString Name;
		Context->GetName(Name);

		return Name;
	}
	// Anything else will be unknown!
	else
	{
		FString UnnamedString = LOCTEXT("UnnamedToolTip", "Unnamed").ToString();
		FString NativeString = IsNativeComponent() ? LOCTEXT("NativeToolTip", "Native ").ToString() : TEXT("");

		return FString::Printf(TEXT("[%s %s]"), *UnnamedString, *NativeString);
	}
}

FText FSubobjectData::GetDragDropDisplayText() const
{
	REDIRECT_TO_ADAPTER(GetDragDropDisplayText);
	
	return FText::FromString(GetDisplayString());
}

FText FSubobjectData::GetDisplayNameContextModifiers(bool bShowNativeComponentNames) const
{
	REDIRECT_TO_ADAPTER(GetDisplayNameContextModifiers, bShowNativeComponentNames);
	
	if(IsActor())
	{
		if(IsChildActor())
		{
			return LOCTEXT("ActorContext_ChildActor", "(Child Actor)");
		}
		else
		{
			if (const AActor* DefaultActor = GetObject<AActor>())
			{
				if (const UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(DefaultActor->GetClass()))
				{
					return LOCTEXT("ActorContext_self", "(Self)");
				}
				else
				{
					return LOCTEXT("ActorContext_Instance", "(Instance)");
				}
			}
		}
	}
	else if(const UActorComponent* Template = GetComponentTemplate())
	{
		FName VariableName = GetVariableName();
		UBlueprint* Blueprint = GetBlueprint();
		UClass* VariableOwner = (Blueprint != nullptr) ? *Blueprint->SkeletonGeneratedClass : nullptr;
		FProperty* VariableProperty = FindFProperty<FProperty>(VariableOwner, VariableName);

		bool const bHasValidVarName = (VariableName != NAME_None);
		bool const bIsArrayVariable = bHasValidVarName && (VariableOwner != nullptr) && VariableProperty && VariableProperty->IsA<FArrayProperty>();
		const bool bIsBlueprintInstanceInherited = GetSCSNode() != nullptr && IsInstancedInheritedComponent();

		if(bIsBlueprintInstanceInherited)
		{
			FStringFormatNamedArguments Args;
            Args.Add(TEXT("InheritedText"), TEXT("(Inherited)"));
            return FText::FromString(FString::Format(TEXT("{InheritedText}"), Args));
		}
		
		// Only display node variable names in the tree if they have not been autogenerated
		if (bHasValidVarName && !bIsArrayVariable)
		{
			const bool bIsNative = IsNativeComponent();
			const bool bIsInherited = IsInheritedSCSNode();

			if ((bIsNative || bIsInherited || bIsBlueprintInstanceInherited) && bShowNativeComponentNames)
			{		
				FStringFormatNamedArguments Args;
				Args.Add(TEXT("InheritedText"), TEXT("(Inherited)"));
				return FText::FromString(FString::Format(TEXT("{InheritedText}"), Args));
			}
		}
	}

	return FText::GetEmpty();
}

FText FSubobjectData::GetDisplayName() const
{
	REDIRECT_TO_ADAPTER(GetDisplayName);
	
	if (bIsChildActor)
	{
		if (const UChildActorComponent* CAC = GetObject<UChildActorComponent>())
		{
			return CAC->GetClass()->GetDisplayNameText();
		}
	}
	if (const AActor* DefaultActor = GetObject<AActor>())
	{
		FString Name;
		UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(DefaultActor->GetClass());
		if (Blueprint != nullptr && !IsInstancedActor())
		{
			Blueprint->GetName(Name);
		}
		else
		{
			Name = DefaultActor->GetActorLabel();
		}
		return FText::FromString(Name);
	}
	else if (const UObject* Context = GetObject())
	{
		FString Name;
		Context->GetName(Name);

		return FText::FromString(Name);
	}

	return LOCTEXT("GetDisplayNameNotOverridden", "Unknown Subobject");
}

FName FSubobjectData::GetVariableName() const
{
	REDIRECT_TO_ADAPTER(GetVariableName);
	
	FName VariableName = NAME_None;

	const USCS_Node* SCS_Node = GetSCSNode();
	const UActorComponent* ComponentTemplate = GetComponentTemplate();

	if (SCS_Node != nullptr)
	{
		// Use the same variable name as is obtained by the compiler
		VariableName = SCS_Node->GetVariableName();
	}
	else if (ComponentTemplate != nullptr)
	{
		// Try to find the component anchor variable name (first looks for an exact match then scans for any matching variable that points to the archetype in the CDO)
		VariableName = FComponentEditorUtils::FindVariableNameGivenComponentInstance(ComponentTemplate);
	}

	return VariableName;
}

FText FSubobjectData::GetSocketName() const
{
	REDIRECT_TO_ADAPTER(GetSocketName);
	
	if (USCS_Node* SCSNode = GetSCSNode())
	{
		return FText::FromName(SCSNode->AttachToName);
	}
	return FText::GetEmpty();
}

FName FSubobjectData::GetSocketFName() const
{
	REDIRECT_TO_ADAPTER(GetSocketFName);

	if (USCS_Node* SCSNode = GetSCSNode())
	{
		return SCSNode->AttachToName;
	}
	return NAME_None;
}

bool FSubobjectData::HasValidSocket() const
{
	REDIRECT_TO_ADAPTER(HasValidSocket);
	
	return GetSCSNode() != nullptr;
}

void FSubobjectData::SetSocketName(FName InNewName)
{
	REDIRECT_TO_ADAPTER(SetSocketName, InNewName);

	if(USCS_Node* SCS = GetSCSNode())
	{
		SCS->AttachToName = InNewName;
	}
}

void FSubobjectData::SetupAttachment(FName SocketName, const FSubobjectDataHandle& AttachParentHandle)
{
	REDIRECT_TO_ADAPTER(SetupAttachment, SocketName, AttachParentHandle);

	USceneComponent* SceneComponentTemplate = Cast<USceneComponent>(GetMutableComponentTemplate());
	if (!SceneComponentTemplate)
	{
		return;
	}
	
	FSubobjectData* AttachParentData = AttachParentHandle.GetData();
	USceneComponent* AttachParent = AttachParentData ?
		Cast<USceneComponent>(AttachParentData->GetMutableComponentTemplate()) :
		SceneComponentTemplate->GetAttachParent();
	
	SceneComponentTemplate->SetupAttachment(AttachParent, NAME_None);
	if(USCS_Node* SCS_Node = GetSCSNode())
	{
		SCS_Node->Modify();
		SCS_Node->AttachToName = NAME_None;
	}
}

FSubobjectDataHandle FSubobjectData::GetRootSubobject() const
{
	FSubobjectDataHandle Current = ParentObjectHandle;
	while(Current.IsValid() && Current.GetSharedDataPtr()->HasParent())
	{
		Current = Current.GetSharedDataPtr()->GetParentHandle();
	}

	return Current;
}

bool FSubobjectData::HasChild(const FSubobjectDataHandle& InChildHandle) const
{
	for(const FSubobjectDataHandle& MyChildHandle : ChildrenHandles)
	{
		if(MyChildHandle == InChildHandle)
		{
			return true;
		}
	}
	return false;
}

FSubobjectDataHandle FSubobjectData::FindChild(const FSubobjectDataHandle& InChildHandle) const
{
	for(const FSubobjectDataHandle& MyChildHandle : ChildrenHandles)
	{
		if(MyChildHandle == InChildHandle)
		{
			return MyChildHandle;
		}		
	}

	return FSubobjectDataHandle::InvalidHandle;
}

FSubobjectDataHandle FSubobjectData::FindChildByObject(UObject* ContextObject) const
{
	REDIRECT_TO_ADAPTER(FindChildByObject, ContextObject)

	if (!ContextObject)
	{
		return FSubobjectDataHandle::InvalidHandle;
	}

	for (const FSubobjectDataHandle& CurrentChild : ChildrenHandles)
	{
		if (FSubobjectData* ChildData = CurrentChild.GetData())
		{
			if (ChildData->GetObject() == ContextObject)
			{
				return CurrentChild;
			}
		}
	}

	return FSubobjectDataHandle::InvalidHandle;
}

bool FSubobjectData::AddChildHandleOnly(const FSubobjectDataHandle& InHandle)
{
	// If we already have this child, then don't both with adding it
	if (HasChild(InHandle) || !InHandle.IsValid() || InHandle == Handle)
	{
		return false;
	}

	ChildrenHandles.Add(InHandle);
	if(FSubobjectData* NewChildData = InHandle.GetData())
	{
		NewChildData->SetParentHandle(Handle);		
	}

	return true;
}

bool FSubobjectData::RemoveChildHandleOnly(const FSubobjectDataHandle& InHandle)
{
	if (HasChild(InHandle))
	{
		ChildrenHandles.Remove(InHandle);
		return true;
	}
	return false;
}

FText FSubobjectData::GetAssetName() const
{
	REDIRECT_TO_ADAPTER(GetAssetName)
	
	UActorComponent* Template = GetMutableComponentTemplate();

	FText AssetName = LOCTEXT("None", "None");
	if (Template)
	{
		UObject* Asset = FComponentAssetBrokerage::GetAssetFromComponent(Template);
		if (Asset != nullptr)
		{
			AssetName = FText::FromString(Asset->GetName());
		}
	}

	return AssetName;
}

FText FSubobjectData::GetAssetPath() const
{
	REDIRECT_TO_ADAPTER(GetAssetPath)

	FText AssetName = LOCTEXT("None", "None");

	UActorComponent* Template = GetMutableComponentTemplate();

	if (Template)
	{
		UObject* Asset = FComponentAssetBrokerage::GetAssetFromComponent(Template);
		if (Asset != nullptr)
		{
			AssetName = FText::FromString(Asset->GetPathName());
		}
	}

	return AssetName;
}

bool FSubobjectData::IsAssetVisible() const
{
	REDIRECT_TO_ADAPTER(IsAssetVisible)

	UActorComponent* Template = GetMutableComponentTemplate();

	if (Template && FComponentAssetBrokerage::SupportsAssets(Template))
	{
		return true;
	}

	return false;
}

FText FSubobjectData::GetToolTipText() const
{
	REDIRECT_TO_ADAPTER(GetToolTipText)

	FText NodeDisplayName = FText::FromString(GetDisplayString());

	if (const UActorComponent* ComponentTemplate = GetComponentTemplate())
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("NodeName"), NodeDisplayName);
		Args.Add(TEXT("ClassName"), GetComponentTemplate()->GetClass()->GetDisplayNameText());

		return FText::Format(LOCTEXT("ComponentTooltip", "{NodeName} ({ClassName})"), Args);
	}

	return NodeDisplayName;
}

FText FSubobjectData::GetMobilityToolTipText() const
{
	REDIRECT_TO_ADAPTER(GetMobilityToolTipText)

	FText MobilityToolTip;
    
    if (const USceneComponent* SceneComponentTemplate = Cast<USceneComponent>(GetComponentTemplate()))
    {
    	if (SceneComponentTemplate->Mobility == EComponentMobility::Movable)
    	{
    		MobilityToolTip = LOCTEXT("MovableMobilityTooltip", "Movable");
    	}
    	else if (SceneComponentTemplate->Mobility == EComponentMobility::Stationary)
    	{
    		MobilityToolTip = LOCTEXT("StationaryMobilityTooltip", "Stationary");
    	}
    	else if (SceneComponentTemplate->Mobility == EComponentMobility::Static)
    	{
    		MobilityToolTip = LOCTEXT("StaticMobilityTooltip", "Static");
    	}
    	else
    	{
    		// make sure we're the mobility type we're expecting (we've handled Movable & Stationary)
    		ensureMsgf(false, TEXT("Unhandled mobility type [%d], is this a new type that we don't handle here?"), SceneComponentTemplate->Mobility.GetValue());
    		MobilityToolTip = LOCTEXT("UnknownMobilityTooltip", "Component with unknown mobility");
    	}
    }
    else
    {
    	MobilityToolTip = LOCTEXT("NoMobilityTooltip", "Non-scene component");
    }

	return MobilityToolTip;
}

FText FSubobjectData::GetComponentEditorOnlyTooltipText() const
{
	REDIRECT_TO_ADAPTER(GetComponentEditorOnlyTooltipText)

	FText ComponentType = LOCTEXT("ComponentEditorOnlyFalse", "False");
	
	if (IsComponent())
	{
		if (const UActorComponent* Template = GetComponentTemplate())
		{
			UBlueprint* Blueprint = GetBlueprint();
			FObjectProperty* Prop = Blueprint ? FindFProperty<FObjectProperty>(Blueprint->SkeletonGeneratedClass, GetVariableName()) : nullptr;

			if(Template->bIsEditorOnly || (Prop && Prop->HasAnyPropertyFlags(CPF_EditorOnly)))
			{
				ComponentType = LOCTEXT("ComponentEditorOnlyTrue", "True");
			}
		}
	}
	
	return ComponentType;
}

FText FSubobjectData::GetIntroducedInToolTipText() const
{
	REDIRECT_TO_ADAPTER(GetIntroducedInToolTipText)

	FText IntroducedInTooltip = LOCTEXT("IntroducedInThisBPTooltip", "this class");

	if (IsInheritedComponent())
	{
		if (const UActorComponent* ComponentTemplate = GetComponentTemplate())
		{
			UClass* BestClass = nullptr;
			AActor* OwningActor = ComponentTemplate->GetOwner();

			if (IsNativeComponent() && (OwningActor != nullptr))
			{
				for (UClass* TestClass = OwningActor->GetClass(); TestClass != AActor::StaticClass(); TestClass = TestClass->GetSuperClass())
				{
					if (FindComponentInstanceInActor(Cast<AActor>(TestClass->GetDefaultObject())))
					{
						BestClass = TestClass;
					}
					else
					{
						break;
					}
				}
			}
			else if (!IsNativeComponent())
			{
				USCS_Node* SCSNode = GetSCSNode();

				if ((SCSNode == nullptr) && (OwningActor != nullptr))
				{
					SCSNode = FindSCSNodeForInstance(ComponentTemplate, OwningActor->GetClass());
				}

				if (SCSNode != nullptr)
				{
					if (UBlueprint* OwningBP = SCSNode->GetSCS()->GetBlueprint())
					{
						BestClass = OwningBP->GeneratedClass;
					}
				}
				else if (OwningActor != nullptr)
				{
					if (UBlueprint* OwningBP = UBlueprint::GetBlueprintFromClass(OwningActor->GetClass()))
					{
						BestClass = OwningBP->GeneratedClass;
					}
				}
			}

			if (BestClass == nullptr)
			{
				if (ComponentTemplate->IsCreatedByConstructionScript()) 
				{
					IntroducedInTooltip = LOCTEXT("IntroducedInUnknownError", "Unknown Blueprint Class (via an Add Component call)");
				} 
				else 
				{
					IntroducedInTooltip = LOCTEXT("IntroducedInNativeError", "Unknown native source (via C++ code)");
				}
			}
			else if (IsInstancedComponent() && ComponentTemplate->CreationMethod == EComponentCreationMethod::Native && !ComponentTemplate->HasAnyFlags(RF_DefaultSubObject))
			{
				IntroducedInTooltip = FText::Format(LOCTEXT("IntroducedInCPPErrorFmt", "{0} (via C++ code)"), FBlueprintEditorUtils::GetFriendlyClassDisplayName(BestClass));
			}
			else if (IsInstancedComponent() && ComponentTemplate->CreationMethod == EComponentCreationMethod::UserConstructionScript)
			{
				IntroducedInTooltip = FText::Format(LOCTEXT("IntroducedInUCSErrorFmt", "{0} (via an Add Component call)"), FBlueprintEditorUtils::GetFriendlyClassDisplayName(BestClass));
			}
			else
			{
				IntroducedInTooltip = FBlueprintEditorUtils::GetFriendlyClassDisplayName(BestClass);
			}
		}
		else
		{
			IntroducedInTooltip = LOCTEXT("IntroducedInNoTemplateError", "[no component template found]");
		}
	}
	else if (IsInstancedComponent())
	{
		IntroducedInTooltip = LOCTEXT("IntroducedInThisActorInstanceTooltip", "this actor instance");
	}

	return IntroducedInTooltip;
}

FText FSubobjectData::GetActorDisplayText() const
{
	REDIRECT_TO_ADAPTER(GetActorDisplayText)

	if (bIsChildActor)
	{
		if (const AActor* ChildActor = GetObject<AActor>())
		{
			return ChildActor->GetClass()->GetDisplayNameText();
		}
	}
	if (const AActor* DefaultActor = GetObject<AActor>())
	{
		FString Name;
		UBlueprint* Blueprint = UBlueprint::GetBlueprintFromClass(DefaultActor->GetClass());
		if (Blueprint != nullptr && !IsInstancedActor())
		{
			Blueprint->GetName(Name);
		}
		else
		{
			Name = DefaultActor->GetActorLabel();
		}
		return FText::FromString(Name);
	}
	
	return FText::GetEmpty();
}

void FSubobjectData::DisposeUnsafe()
{
	Handle = FSubobjectDataHandle();
	ParentObjectHandle = FSubobjectDataHandle();
}

void FSubobjectData::DisconnectFromParent()
{
	if (ParentObjectHandle.IsValid())
	{
		ParentObjectHandle.GetData()->RemoveChildHandleOnly(GetHandle());
	}
	ParentObjectHandle = FSubobjectDataHandle::InvalidHandle;
}

const UObject* FSubobjectData::GetObject(bool bEvenIfPendingKill) const
{
	return WeakObjectPtr.Get(bEvenIfPendingKill);
}

const TInstancedStruct<FSubobjectDataSubsystemContextDataBase>& FSubobjectData::GetContextData() const
{
	return ContextData;
}

bool FSubobjectData::IsInstancedActor() const
{
	REDIRECT_TO_ADAPTER(IsInstancedActor)

	if (const AActor* Actor = GetObject<AActor>())
	{
		return !Actor->IsTemplate();
	}

	return false;
}

bool FSubobjectData::IsNativeComponent() const
{
	REDIRECT_TO_ADAPTER(IsNativeComponent)

	if (const UActorComponent* Template = GetComponentTemplate())
	{
		return Template->CreationMethod == EComponentCreationMethod::Native && GetSCSNode() == nullptr;
	}
	
	return false;
}

bool FSubobjectData::IsBlueprintInheritedComponent() const
{
	REDIRECT_TO_ADAPTER(IsBlueprintInheritedComponent)

	if (const UActorComponent* Template = GetComponentTemplate())
	{
		if(GetSCSNode() != nullptr)
		{
			return IsInheritedSCSNode();
		}
	}

	return false;
}

bool FSubobjectData::IsInheritedComponent() const
{
	REDIRECT_TO_ADAPTER(IsInheritedComponent)

	// This covers a component that is added via blueprints
	if(GetSCSNode() != nullptr)
	{
		return IsInheritedSCSNode();
	}
	else if (const UActorComponent* ComponentTemplate = GetComponentTemplate())
	{
		return IsNativeComponent() || ComponentTemplate->CreationMethod != EComponentCreationMethod::Instance;
	}

	return false;
}

bool FSubobjectData::IsSceneComponent() const
{
	REDIRECT_TO_ADAPTER(IsSceneComponent)

	return Cast<USceneComponent>(GetComponentTemplate()) != nullptr;
}

bool FSubobjectData::IsRootComponent() const
{
	REDIRECT_TO_ADAPTER(IsRootComponent)

	const UActorComponent* ComponentTemplate = GetComponentTemplate();
	
	if(!ComponentTemplate)
	{
		return false;
	}
	
	const AActor* CDO = ComponentTemplate ? ComponentTemplate->GetOwner() : nullptr;
	
	if(CDO && (IsInstancedComponent() || IsInheritedComponent()))
	{
		return CDO->GetRootComponent() == ComponentTemplate;
	}
	
	bool bIsRoot = true;
	if(USCS_Node* SCS_Node = GetSCSNode())
	{
		if(USimpleConstructionScript* SCS = SCS_Node->GetSCS())
		{
			// Evaluate to TRUE if we have an SCS node reference, it is contained in the SCS root set and does not have an external parent
			bIsRoot = SCS->GetRootNodes().Contains(SCS_Node) && SCS_Node->ParentComponentOrVariableName == NAME_None;
		}
	}
	else if(ComponentTemplate && CDO)
	{
		// Evaluate to TRUE if we have a valid component reference that matches the native root component
		bIsRoot = (ComponentTemplate == CDO->GetRootComponent());
	}
	
	return bIsRoot;
}

bool FSubobjectData::IsDefaultSceneRoot() const
{
	REDIRECT_TO_ADAPTER(IsDefaultSceneRoot)
	
	// If this is a scene component and is instanced, then it will have a specific name
	const USceneComponent* SceneComponent = Cast<USceneComponent>(GetComponentTemplate());
	if (SceneComponent && !SceneComponent->IsTemplate())
	{
		const AActor* OwningActor = SceneComponent->GetAttachmentRootActor();		
		return
			SceneComponent->GetFName() == USceneComponent::GetDefaultSceneRootVariableName() || 
			(OwningActor && SceneComponent == OwningActor->GetRootComponent());
	}
	// If this isn't a scene component, then we can check an SCS node for a flag. 
	else if (const USCS_Node* SCS_Node = GetSCSNode())
	{
		if (USimpleConstructionScript* SCS = SCS_Node->GetSCS())
		{
			const USceneComponent* SCS_Root = SCS->GetSceneRootComponentTemplate();

			return (SCS_Node == SCS->GetDefaultSceneRootNode()) || (SceneComponent && (SCS_Root == SceneComponent));
		}
	}
	// As a last resort check the owning actor to see if the root component matches up with this
	// This will be the case for native subobjects
	else if(SceneComponent)
	{
		const AActor* Owner = SceneComponent->GetOwner();
		return Owner && Owner->GetRootComponent() == SceneComponent;
	}

	// Nothing else can be a DefaultSceneRoot
	return false;
}

bool FSubobjectData::SceneRootHasDefaultName() const
{
	REDIRECT_TO_ADAPTER(SceneRootHasDefaultName)

	const UActorComponent* Template = GetComponentTemplate();
	const FName TemplateName = Template ? Template->GetFName() : NAME_None;

	// Only the first default scene root can be attached to an actor, so this will 
	// rule out false positives of other subobjects that may have been named the same thing
	// or if the first default scene root was duplicated
	const FSubobjectData* ParentData = GetParentHandle().GetData();
	const bool bIsAttachedToActor = ParentData ? ParentData->IsActor() : false;

	return bIsAttachedToActor && TemplateName.ToString().StartsWith(USceneComponent::GetDefaultSceneRootVariableName().ToString());
}

bool FSubobjectData::IsComponent() const
{
	REDIRECT_TO_ADAPTER(IsComponent)

	// Check if we are pointing to a component
	return GetComponentTemplate() != nullptr;
}

bool FSubobjectData::IsChildActor() const
{
	REDIRECT_TO_ADAPTER(IsChildActor)

	return bIsChildActor;
}

bool FSubobjectData::IsChildActorSubtreeObject() const
{
	REDIRECT_TO_ADAPTER(IsChildActorSubtreeObject)

	const FSubobjectDataHandle& RootActor = GetRootSubobject();
	if (const FSubobjectData* RootData = RootActor.GetData())
	{
		return RootData->IsChildActor();
	}
	return false;
}

bool FSubobjectData::IsRootActor() const
{
	REDIRECT_TO_ADAPTER(IsRootActor)
	
	// This is the root actor if it points to an AActor and has no parent
	if(const AActor* Actor = GetObject<AActor>())
	{
		return !ParentObjectHandle.IsValid();
	}
	return false;
}

bool FSubobjectData::IsActor() const
{
	REDIRECT_TO_ADAPTER(IsActor)
	
	return GetObject<AActor>() != nullptr;
}

bool FSubobjectData::IsInstancedInheritedComponent() const
{
	REDIRECT_TO_ADAPTER(IsInstancedInheritedComponent)

	if(!IsComponent())
	{
		return false;
	}
	FSubobjectDataHandle CurrentHandle = ParentObjectHandle;
	FSubobjectData* CurrentData = CurrentHandle.GetData();
			
	while(CurrentHandle.IsValid() && CurrentData && !CurrentData->IsActor())
	{
		CurrentHandle = CurrentData->GetParentHandle();
		CurrentData = CurrentHandle.GetData(); 
	}

	return CurrentData && CurrentData->IsInstancedActor();
}

bool FSubobjectData::IsAttachedTo(const FSubobjectDataHandle& InHandle) const
{
	REDIRECT_TO_ADAPTER(IsAttachedTo, InHandle)

	FSubobjectDataHandle TestParentHandle = ParentObjectHandle;

	while (TestParentHandle.IsValid())
	{
		if (TestParentHandle == InHandle)
		{
			return true;
		}
		const FSubobjectData* TestParentData = TestParentHandle.GetData();
		TestParentHandle = TestParentData ? TestParentData->GetParentHandle() : FSubobjectDataHandle::InvalidHandle;
	}

	return false;
}

AActor* FSubobjectData::GetMutableActorContext()
{
	if(AActor* Actor = GetMutableObject<AActor>())
	{
		return Actor;
	}
	else if(UActorComponent* Component = GetMutableComponentTemplate())
	{
		return Component->GetOwner();
	}
	else if(UBlueprint* BP = GetBlueprint())
	{
		return BP->GeneratedClass ? BP->GeneratedClass->GetDefaultObject<AActor>() : nullptr;
	}
	return nullptr;
}

USCS_Node* FSubobjectData::FindSCSNodeForInstance(const UActorComponent* InstanceComponent, UClass* ClassToSearch)
{
	if ((ClassToSearch != nullptr) && InstanceComponent->IsCreatedByConstructionScript())
	{
		USCS_Node* Result = nullptr;
		UBlueprintGeneratedClass::ForEachGeneratedClassInHierarchy(ClassToSearch, [InstanceComponent, &Result](const UBlueprintGeneratedClass* BPGClass)
			{
				if (BPGClass->SimpleConstructionScript != nullptr)
				{
					Result = BPGClass->SimpleConstructionScript->FindSCSNode(InstanceComponent->GetFName());
					if (Result != nullptr)
					{
						// Stop traversing when we find the node.
						return false;
					}
				}

				return true;
			});
		return Result;
	}

	return nullptr;
}

bool FSubobjectData::AttemptToSetSCSNode()
{
	if (USCS_Node* PossibleSCS = Cast<USCS_Node>(WeakObjectPtr.Get()))
	{
		WeakObjectPtr = PossibleSCS->ComponentTemplate;
		SCSNodePtr = PossibleSCS;
		return true;
	}
	// If this is an instanced component, then we can find it's SCS node
	else if (IsInstancedComponent())
	{
		const UActorComponent* Template = GetComponentTemplate();
		if (Template->GetOwner())
		{
			SCSNodePtr = FindSCSNodeForInstance(Template, Template->GetOwner()->GetClass());
			if (SCSNodePtr.IsValid())
			{
				WeakObjectPtr = SCSNodePtr->ComponentTemplate;
				return true;
			}
		}
	}
	return false;
}

USCS_Node* FSubobjectData::GetSCSNode(bool bEvenIfPendingKill) const
{
	// @todo Deprecate everything related to SCS nodes that could possibly be public facing.
	return SCSNodePtr.IsValid() ? SCSNodePtr.Get() : Cast<USCS_Node>(WeakObjectPtr.Get(bEvenIfPendingKill));
}

bool FSubobjectData::IsInheritedSCSNode() const
{
	return bIsInheritedSCS;
}

#undef LOCTEXT_NAMESPACE
