// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubobjectDataAdapterBase.h"

#include "DataStorage/Features.h"
#include "DataStorage/ScopedRowHandle.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "SubobjectDataHandle.h"
#include "SubobjectDataTedsFactory.h"

#define LOCTEXT_NAMESPACE "FSubobjectDataBehaviorBase"

FSubobjectDataSubsystemContextData_SingleUObjectContextObject::FSubobjectDataSubsystemContextData_SingleUObjectContextObject()
{
}

FSubobjectDataSubsystemContextData_SingleUObjectContextObject::FSubobjectDataSubsystemContextData_SingleUObjectContextObject(UObject* ContextObject)
	: Object(ContextObject)
{
}

FSubobjectDataSubsystemContextData_TedsRow::FSubobjectDataSubsystemContextData_TedsRow()
{
}

FSubobjectDataSubsystemContextData_TedsRow::FSubobjectDataSubsystemContextData_TedsRow(UE::Editor::DataStorage::RowHandle InRowHandle)
{
	using namespace UE::Editor::DataStorage;
	if (InRowHandle == InvalidRowHandle)
	{
		return;
	}
	ICoreProvider* EditorDataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!ensure(EditorDataStorage))
	{
		return;
	}
	Row = MakeShared<FScopedRow>(EditorDataStorage, InRowHandle);
}

UE::Editor::DataStorage::TableHandle FSubobjectDataSubsystemContextData_TedsRow::GetSubobjectItemTable(
	const UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	const USubobjectDataTedsFactory* Factory = DataStorage.FindFactory<USubobjectDataTedsFactory>();
	return Factory->GetSubobjectDataTableHandle();
}

FSubobjectDataSubsystemAdapterBase::~FSubobjectDataSubsystemAdapterBase() = default;

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_GatherSubobjectData(const UE::Editor::FSubobjectEditorContext& Context) const
{
	return false;
}

void FSubobjectDataSubsystemAdapterBase::GatherSubobjectData(const UE::Editor::FSubobjectEditorContext& Context, TArray<FSubobjectDataHandle>& OutArray, TFunctionRef<FSubobjectDataHandle()> CreateSubobjectData) const
{
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_FindSceneRootForSubobject(const FSubobjectDataHandle& InHandle) const
{
	return false;
}

FSubobjectDataHandle FSubobjectDataSubsystemAdapterBase::FindSceneRootForSubobject(const FSubobjectDataHandle& InHandle) const
{
	return FSubobjectDataHandle();
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_DeleteSubobjects(const FSubobjectDataHandle& ContextHandle,
                                                                       const TArray<FSubobjectDataHandle>& SubobjectsToDelete, UBlueprint* BPContext, bool bForce) const
{
	return false;
}

int32 FSubobjectDataSubsystemAdapterBase::DeleteSubobjects(
	const FSubobjectDataHandle& ContextHandle,
	const TArray<FSubobjectDataHandle>& SubobjectsToDelete) const
{
	return 0;
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_CanCopySubobjects(const TArray<FSubobjectDataHandle>& Handles) const
{
	return false;
}

bool FSubobjectDataSubsystemAdapterBase::CanCopySubobjects(const TArray<FSubobjectDataHandle>& Handles) const
{
	return false;
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_CopySubobjects(
	const TArray<FSubobjectDataHandle>& Handles,
	UBlueprint* BpContext) const
{
	return false;
}

void FSubobjectDataSubsystemAdapterBase::CopySubobjects(
	const TArray<FSubobjectDataHandle>& Handles,
	UBlueprint* BpContext) const
{
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_CanPasteSubobjects(const FSubobjectDataHandle& RootHandle, UBlueprint* BPContext) const
{
	return false;
}

bool FSubobjectDataSubsystemAdapterBase::CanPasteSubobjects(const FSubobjectDataHandle& RootHandle, UBlueprint* Blueprint) const
{
	return false;
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_PasteSubobjects(const FSubobjectDataHandle& RootHandle,
	const TArray<FSubobjectDataHandle>& NewParentHandles, UBlueprint* BPContext) const
{
	return false;
}

void FSubobjectDataSubsystemAdapterBase::PasteSubobjects(const FSubobjectDataHandle& RootHandle, const TArray<FSubobjectDataHandle>& NewParentHandles,
	UBlueprint* Blueprint, TArray<FSubobjectDataHandle>& OutPastedHandles) const
{
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_CanDuplicateSubobjects(
	const FSubobjectDataHandle& Context,
	const TArray<FSubobjectDataHandle>& SubobjectsToDup) const
{
	return false;
}

bool FSubobjectDataSubsystemAdapterBase::CanDuplicateSubobjects(
	const FSubobjectDataHandle& Context,
	const TArray<FSubobjectDataHandle>& SubobjectsToDup) const
{
	return false;
}

bool FSubobjectDataSubsystemAdapterBase::CanIntercept_DuplicateSubobjects(
	const FSubobjectDataHandle& Context,
	const TArray<FSubobjectDataHandle>& SubobjectsToDup,
	UBlueprint* BpContext) const
{
	return false;
}

void FSubobjectDataSubsystemAdapterBase::DuplicateSubobjects(
	const FSubobjectDataHandle& Context,
	const TArray<FSubobjectDataHandle>& SubobjectsToDup,
	UBlueprint* BpContext,
	TArray<FSubobjectDataHandle>& OutNewSubobjects) const
{
}

FSubobjectDataAdapterBase::~FSubobjectDataAdapterBase() = default;

bool FSubobjectDataAdapterBase::CanEdit(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::CanDelete(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::CanDuplicate(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::CanCopy(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::CanReparent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::CanRename(const FSubobjectData& Data) const
{
	return false;
}

const UObject* FSubobjectDataAdapterBase::GetObject(const FSubobjectData& Data, bool bEventIfPendingKill) const
{
	return nullptr;
}

const UObject* FSubobjectDataAdapterBase::GetObjectForBlueprint(const FSubobjectData& Data, UBlueprint* Blueprint) const
{
	return nullptr;
}

const UActorComponent* FSubobjectDataAdapterBase::GetComponentTemplate(const FSubobjectData& Data, bool bEvenIfPendingKill) const
{
	return nullptr;
}

const UActorComponent* FSubobjectDataAdapterBase::FindComponentInstanceInActor(const FSubobjectData& Data, const AActor* InActor) const
{
	return nullptr;
}

UActorComponent* FSubobjectDataAdapterBase::FindMutableComponentInstanceInActor(const FSubobjectData& Data, const AActor* InActor) const
{
	return nullptr;
}

UBlueprint* FSubobjectDataAdapterBase::GetBlueprint(const FSubobjectData& Data) const
{
	return nullptr;
}

UBlueprint* FSubobjectDataAdapterBase::GetBlueprintBeingEdited(const FSubobjectData& Data) const
{
	return nullptr;
}

bool FSubobjectDataAdapterBase::IsInstancedComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsInstancedActor(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsNativeComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsBlueprintInheritedComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsInheritedComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsSceneComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsRootComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsDefaultSceneRoot(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::SceneRootHasDefaultName(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsChildActor(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsChildActorSubtreeObject(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsRootActor(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsActor(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsInstancedInheritedComponent(const FSubobjectData& Data) const
{
	return false;
}

bool FSubobjectDataAdapterBase::IsAttachedTo(const FSubobjectData& Data, const FSubobjectDataHandle& InHandle) const
{
	return false;
}

FString FSubobjectDataAdapterBase::GetDisplayString(const FSubobjectData& Data, bool bShowNativeComponentNames) const
{
	return TEXT("");
}

FText FSubobjectDataAdapterBase::GetDragDropDisplayText(const FSubobjectData& Data) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetDisplayNameContextModifiers(const FSubobjectData& Data,
	bool bShowNativeComponentNames) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetDisplayName(const FSubobjectData& Data) const
{
	return FText();
}

FName FSubobjectDataAdapterBase::GetVariableName(const FSubobjectData& Data) const
{
	return FName();
}

FText FSubobjectDataAdapterBase::GetSocketName(const FSubobjectData& Data) const
{
	return FText();
}

FName FSubobjectDataAdapterBase::GetSocketFName(const FSubobjectData& Data) const
{
	return FName();
}

bool FSubobjectDataAdapterBase::HasValidSocket(const FSubobjectData& Data) const
{
	return false;
}

void FSubobjectDataAdapterBase::SetSocketName(FSubobjectData& Data, FName InNewName) const
{
}

void FSubobjectDataAdapterBase::SetupAttachment(FSubobjectData& Data, FName SocketName, const FSubobjectDataHandle& AttachParentHandle) const
{
}

FSubobjectDataHandle FSubobjectDataAdapterBase::FindChildByObject(const FSubobjectData& Data, UObject* ContextObject) const
{
	return FSubobjectDataHandle();
}

FText FSubobjectDataAdapterBase::GetAssetName(const FSubobjectData& Data) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetAssetPath(const FSubobjectData& Data) const
{
	return FText();
}

bool FSubobjectDataAdapterBase::IsAssetVisible(const FSubobjectData& Data) const
{
	return false;
}

FText FSubobjectDataAdapterBase::GetToolTipText(const FSubobjectData& Data) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetMobilityToolTipText(const FSubobjectData& Data) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetComponentEditorOnlyTooltipText(const FSubobjectData& Data) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetIntroducedInToolTipText(const FSubobjectData& Data) const
{
	return FText();
}

FText FSubobjectDataAdapterBase::GetActorDisplayText(const FSubobjectData& Data) const
{
	return FText();
}

#undef LOCTEXT_NAMESPACE
