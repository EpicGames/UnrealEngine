// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubobjectDataBlueprintFunctionLibrary.h"

#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubobjectDataBlueprintFunctionLibrary)

void USubobjectDataBlueprintFunctionLibrary::GetData(const FSubobjectDataHandle& DataHandle, FSubobjectData& OutData)
{
	// Copy the underlying subobject data - probably to the stack so that script can manipulate it
	TSharedPtr<FSubobjectData> DataPtr = DataHandle.GetSharedDataPtr();
	if(DataPtr.IsValid())
	{
		OutData = *DataPtr.Get();
	}
}

const UObject* USubobjectDataBlueprintFunctionLibrary::GetObject(const FSubobjectData& Data, bool bEvenIfPendingKill)
{
	return Data.GetObject(bEvenIfPendingKill);
}

const UObject* USubobjectDataBlueprintFunctionLibrary::GetAssociatedObject(const FSubobjectData& Data)
{
	if(!Data.IsValid())
	{
		return nullptr;
	}

	FSubobjectDataHandle Root = Data.GetRootSubobject();
	if(Root != FSubobjectDataHandle())
	{
		// resolve our object within the context of our root:
		const UObject* Object = Root.GetData()->GetObject();
		if(const UBlueprint* BP = Cast<UBlueprint>(Object))
		{
			return Data.GetObjectForBlueprint(const_cast<UBlueprint*>(BP));
		}
		else if(const AActor* ActorContext = Cast<AActor>(Object))
		{
			if(const UObject* Component = Data.FindComponentInstanceInActor(ActorContext))
			{
				return Component;
			}
		}
	}

	// we couldn't figure out the root object, hopefully the object 
	// ptr on the subobject data is not something silly:
	const UObject* Object = Data.GetObject();
	if(const UBlueprint* BP = Cast<UBlueprint>(Object))
	{
		return BP->GeneratedClass ? BP->GeneratedClass->GetDefaultObject(false) : nullptr;
	}
	else
	{
		return Object;
	}

}

const UObject* USubobjectDataBlueprintFunctionLibrary::GetObjectForBlueprint(const FSubobjectData& Data, UBlueprint* Blueprint) 
{
	return Data.GetObjectForBlueprint(Blueprint); 
}
