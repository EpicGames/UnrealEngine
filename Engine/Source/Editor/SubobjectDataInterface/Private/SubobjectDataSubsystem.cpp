// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubobjectDataSubsystem.h"
#include "UnrealEngine.h"				// GEngine for subsystems

#if WITH_EDITOR
#include "LevelEditor.h"				// LevelEditor.BroadcastComponentsEdited
#endif	// WITH_EDITOR

#include "Editor/EditorEngine.h"		// FActorLabelUtilities
#include "ComponentAssetBroker.h"		// FComponentAssetBrokerage
#include "Serialization/ArchiveReplaceOrClearExternalReferences.h"

#include "BlueprintEditorSettings.h"	// bHideConstructionScriptComponentsInDetailsView
#include "Kismet2/ComponentEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"	// IsClassAllowed
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ClassViewerFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"

#include "GameProjectGenerationModule.h"	// Adding new component classes
#include "GameProjectUtils.h"
#include "SourceCodeNavigation.h"
#include "AddToProjectConfig.h"

#include "Serialization/ObjectWriter.h"
#include "Serialization/ObjectReader.h"

#include "Kismet2/CompilerResultsLog.h"		// Adding compiler errors to nodes that have their variables deleted
#include "K2Node_ComponentBoundEvent.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementHierarchyInterface.h"

#include "Engine/SCS_Node.h"		// #TODO_BH  We need to remove this when the actual subobject refactor happens
#include "Logging/MessageLog.h"
#include "StructUtils/StructView.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubobjectDataSubsystem)

#define LOCTEXT_NAMESPACE "SubobjectDataInterface"

DEFINE_LOG_CATEGORY_STATIC(LogSubobjectSubsystem, Log, All);

namespace UE::SubobjectDataSubsystem
{
	bool bForcePastedComponentsToSCS = true;
	FAutoConsoleVariableRef CVarAudioShapesEnabled(
		TEXT("bp.bForcePastedComponentsToSCS"),
		bForcePastedComponentsToSCS,
		TEXT("Setting this to True will change instanced components pasted into blueprints to be SCS components"),
		ECVF_Default);
}


/** Notify the Level Editor that there have been subobject changes to an instance and it needs to be refreshed */
static void BroadcastInstanceChanges()
{
#if WITH_EDITOR
	FLevelEditorModule& LevelEditor = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.BroadcastComponentsEdited();
#endif	// WITH_EDITOR
}

//////////////////////////////////////////////
// USubobjectDataSubsystem

void USubobjectDataSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
}

void USubobjectDataSubsystem::Deinitialize()
{
}

USubobjectDataSubsystem* USubobjectDataSubsystem::Get()
{
	return GEngine->GetEngineSubsystem<USubobjectDataSubsystem>();
}

FSubobjectDataAdapterHandle USubobjectDataSubsystem::RegisterSubobjectDataAdapter(
	FSubobjectDataAdapterFilter InSubobjectDataAdapterFilter,
	TUniquePtr<FSubobjectDataAdapterBase> InDataAdapter)
{
	FSubobjectDataAdapterHandle Handle;
	// Pre-increment so that invalid handles are 0
	Handle.Opaque = ++DataAdapterHandleIndex;

	DataAdapters.Emplace(FSubobjectDataAdapterEntry
		{
			.Adapter = MoveTemp(InDataAdapter),
			.Handle = Handle,
			.Filter = MoveTemp(InSubobjectDataAdapterFilter)
		});
	return Handle;
}

void USubobjectDataSubsystem::UnregisterSubobjectDataAdapter(FSubobjectDataAdapterHandle Handle)
{
	DataAdapters.RemoveAll([Handle](const FSubobjectDataAdapterEntry& Entry)
		{
			return Entry.Handle.Opaque == Handle.Opaque;
		});
}

const FSubobjectDataAdapterBase* USubobjectDataSubsystem::FindDataAdapter(const FSubobjectData& SubobjectData) const
{
	for (const FSubobjectDataAdapterEntry& Entry : DataAdapters)
	{
		if (Entry.Filter(SubobjectData))
		{
			return Entry.Adapter.Get();
		}
	}
	return nullptr;
}


FSubobjectDataSubsystemAdapterHandle USubobjectDataSubsystem::RegisterSubsystemAdapter(
	TUniquePtr<FSubobjectDataSubsystemAdapterBase> InAdapter)
{
	FSubobjectDataSubsystemAdapterHandle Handle;
	// Pre-increment so that invalid handles are 0
	Handle.Opaque = ++SubsystemAdapterHandleIndex;
	
	SubsystemAdapters.Emplace(FSubsystemAdapterEntry
		{
			.Adapter = MoveTemp(InAdapter),
			.Handle = Handle
		});
	return Handle;
}

void USubobjectDataSubsystem::UnregisterSubsystemAdapter(FSubobjectDataSubsystemAdapterHandle Handle)
{
	SubsystemAdapters.RemoveAll([Handle](const FSubsystemAdapterEntry& Entry)
	{
		return Entry.Handle.Opaque == Handle.Opaque;
	});
}

void USubobjectDataSubsystem::K2_GatherSubobjectDataForBlueprint(UBlueprint* Context, TArray<FSubobjectDataHandle>& OutArray)
{
	// Return the current CDO that was last generated for the class
	if (Context != nullptr && Context->GeneratedClass != nullptr)
	{
		GatherSubobjectData(Context->GeneratedClass->GetDefaultObject(), OutArray);
	}
}

void USubobjectDataSubsystem::K2_GatherSubobjectDataForInstance(AActor* Context, TArray<FSubobjectDataHandle>& OutArray)
{
	return GatherSubobjectData(Context, OutArray);
}

FSubobjectDataHandle USubobjectDataSubsystem::FindOrCreateAttachParentForComponent(UActorComponent* InActorComponent, const FSubobjectDataHandle& ActorRootHandle, TArray<FSubobjectDataHandle>& ExistingHandles)
{
	check(InActorComponent != nullptr);
	const FSubobjectData* ActorRootData = ActorRootHandle.GetData();

	USceneComponent* SceneComponent = Cast<USceneComponent>(InActorComponent);
	if (SceneComponent == nullptr)
	{
		check(ActorRootData->IsValid());
		check(ActorRootData->IsActor());
		return ActorRootData->GetHandle();
	}

	auto FindExistingHandle = [&ExistingHandles](USceneComponent* ComponentToFind) -> FSubobjectDataHandle
	{
		for (const FSubobjectDataHandle& Handle : ExistingHandles)
		{
			const FSubobjectData* Data = Handle.GetData();
			if (Data && Data->GetComponentTemplate() == ComponentToFind)
			{
				return Handle;
			}
		}
		return FSubobjectDataHandle::InvalidHandle;
	};

	FSubobjectDataHandle ParentHandle;
	if (SceneComponent->GetAttachParent() != nullptr)
	{
		// Attempt to find the parent node in the current tree
		ParentHandle = FindExistingHandle(SceneComponent->GetAttachParent());
		if (!ParentHandle.IsValid())
		{
			// If the actual attach parent wasn't found, attempt to find its archetype.
			// This handles the BP editor case where we might add UCS component nodes taken
			// from the preview actor instance, which are not themselves template objects.
			ParentHandle = FindExistingHandle(Cast<USceneComponent>(SceneComponent->GetAttachParent()->GetArchetype()));
			if (!ParentHandle.IsValid())
			{
				// Recursively add the parent handle to the tree if it does not exist yet
				ParentHandle = FactoryCreateSubobjectDataWithParent(SceneComponent->GetAttachParent(), FindOrCreateAttachParentForComponent(
					SceneComponent->GetAttachParent(),
					ActorRootHandle,
					ExistingHandles)
				);

				if (ParentHandle.IsValid())
				{
					ExistingHandles.Add(ParentHandle);
				}				
			}
		}
	}

	// Use the default scene root node as a backup
	if (!ParentHandle.IsValid())
	{
		ParentHandle = FindSceneRootForSubobject(ActorRootData->GetHandle());
	}

	// The actor doesn't have a root component yet
	if (!ParentHandle.IsValid())
	{
		ParentHandle = ActorRootData->GetHandle();
	}

	return ParentHandle;
}

FSubobjectDataHandle USubobjectDataSubsystem::FindAttachParentForInheritedComponent(UActorComponent* InActorComponent, const FSubobjectDataHandle& InStartData, UBlueprint* BP)
{
	FSubobjectDataHandle OutHandle = FSubobjectDataHandle::InvalidHandle;

	if (InActorComponent)
	{
		if (const FSubobjectData* ActorRootData = InStartData.GetData())
		{
			// Check to see if the given component template matches the given tree node
			// 
			// For certain node types, GetOrCreateEditableComponentTemplate() will handle retrieving 
			// the "OverridenComponentTemplate" which may be what we're looking for in some 
			// cases; if not, then we fall back to just checking GetComponentTemplate()
			if (ActorRootData->GetObjectForBlueprint(BP) == InActorComponent)
			{
				OutHandle = ActorRootData->GetHandle();
			}
			else if (ActorRootData->GetComponentTemplate() == InActorComponent)
			{
				OutHandle = ActorRootData->GetHandle();
			}
			else
			{
				// Recursively search for the node in our child set
				OutHandle = ActorRootData->FindChildByObject(InActorComponent);
				if (!OutHandle.IsValid())
				{
					const TArray<FSubobjectDataHandle>& ChildHandles = ActorRootData->GetChildrenHandles();
					for (const FSubobjectDataHandle& Handle : ChildHandles)
					{
						OutHandle = FindAttachParentForInheritedComponent(InActorComponent, Handle, BP);
						if (OutHandle.IsValid())
						{
							break;
						}
					}
				}
			}
		}
	}

	return OutHandle;
}

void USubobjectDataSubsystem::GatherSubobjectData(UObject* ObjectContext, TArray<FSubobjectDataHandle>& OutArray)
{
	UE::Editor::FSubobjectEditorContext Context;
	Context.ObjectContext = ObjectContext;
	GatherSubobjectData(Context, OutArray);
}

void USubobjectDataSubsystem::GatherSubobjectData(const UE::Editor::FSubobjectEditorContext& Context, TArray<FSubobjectDataHandle>& OutArray)
{
	OutArray.Reset();	

	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_GatherSubobjectData(Context))
		{
			return Entry.Adapter->GatherSubobjectData(
				Context,
				OutArray,
				[this]() { return this->CreateSubobjectData(); });
		}
	}
	
	if (!Context.ObjectContext)
	{
		UE_LOGF(LogSubobjectSubsystem, Warning, "Could not gather subobject data, there was no context given!");
		return;
	}

	const AActor* ActorContext = Cast<AActor>(Context.ObjectContext);
	if(!ActorContext)
	{
		return;
	}

	FSubobjectDataHandle RootActorHandle = CreateSubobjectData(Context.ObjectContext);
	OutArray.Add(RootActorHandle);
	const FSubobjectData* RootActorDataPtr = RootActorHandle.GetData();
	check(RootActorDataPtr);

	// Is the root an actor? 
	const bool bIsInstanced = RootActorDataPtr->IsInstancedActor();

	// If the actor is not instanced, then we are dealing with a BP class
	if (!bIsInstanced)
	{
		// get all the components
		TArray<UActorComponent*> Components;
		ActorContext->GetComponents(Components);

		USceneComponent* RootComponent = ActorContext->GetRootComponent();
		FSubobjectDataHandle RootComponentHandle;
		if (RootComponent != nullptr)
		{
			Components.Remove(RootComponent);
			const FSubobjectDataHandle& ParentHandle = FindOrCreateAttachParentForComponent(RootComponent, RootActorHandle, OutArray);
			RootComponentHandle = FactoryCreateSubobjectDataWithParent(RootComponent, ParentHandle);
			OutArray.Add(RootComponentHandle);
		}

		// The components array will be populated with any natively added components
		// from the constructor/ObjectInitializer
		// These components will all be attached to the root component if it exists
		for (UActorComponent* Component : Components)
		{
			const FSubobjectDataHandle& ParentHandle = FindOrCreateAttachParentForComponent(Component, RootActorHandle, OutArray);

			FSubobjectDataHandle NewComponentHandle = FactoryCreateSubobjectDataWithParent(
					/* Context = */ Component,
					/* ParentHandle = */ ParentHandle
			);

			ensureMsgf(NewComponentHandle.IsValid(), TEXT("Gathering of native components failed!"));
			OutArray.Add(NewComponentHandle);
		}

		// If it's a Blueprint-generated class, also get the inheritance stack
		TArray<UBlueprintGeneratedClass*> ParentBPStack;
		UBlueprint::GetBlueprintHierarchyFromClass(ActorContext->GetClass(), ParentBPStack);

		UBlueprint* ActorBP = (ParentBPStack.Num() > 0 && ParentBPStack[0]) ? Cast<UBlueprint>(ParentBPStack[0]->ClassGeneratedBy) : nullptr;
		ensure(ActorBP);

		// Add the full SCS tree node hierarchy (including SCS nodes inherited from parent blueprints)
		for (int32 StackIndex = ParentBPStack.Num() - 1; StackIndex >= 0; --StackIndex)
		{
			USimpleConstructionScript* ParentSCS = ParentBPStack[StackIndex] ? ParentBPStack[StackIndex]->SimpleConstructionScript.Get() : nullptr;

			if (ParentSCS)
			{
				const TArray<USCS_Node*>& SCS_RootNodes = ParentSCS->GetRootNodes();
				for (int32 NodeIndex = 0; NodeIndex < SCS_RootNodes.Num(); ++NodeIndex)
				{
					USCS_Node* SCS_Node = SCS_RootNodes[NodeIndex];
					check(SCS_Node != nullptr);

					// Create a new subobject whose parent 
					FSubobjectDataHandle NewHandle;
					FSubobjectData* NewData = nullptr;
					// If this SCS node has a parent component, then add it to that
					if (SCS_Node->ParentComponentOrVariableName != NAME_None)
					{
						if (USceneComponent* ParentComponent = SCS_Node->GetParentComponentTemplate(ActorBP))
						{
							FSubobjectDataHandle ParentHandle = FindAttachParentForInheritedComponent(ParentComponent, RootActorHandle, ActorBP);
							if (ParentHandle.IsValid())
							{
								NewHandle = FactoryCreateInheritedBpSubobject(SCS_Node, ParentHandle, /* bIsInherited = */ StackIndex > 0, OutArray);
							}
						}
					}
					// Otherwise add a node parents to the root actor
					else
					{
						FSubobjectDataHandle& ParentHandle = RootActorHandle;
						// Only a valid scene component can be used as the parent handle for a subobject, otherwise it should be attached to the root actor
						if (RootComponentHandle.IsValid() && SCS_Node->ComponentTemplate && SCS_Node->ComponentTemplate->IsA<USceneComponent>())
						{
							ParentHandle = RootComponentHandle;
						}
						NewHandle = FactoryCreateInheritedBpSubobject(SCS_Node, ParentHandle, /* bIsInherited = */ StackIndex > 0, OutArray);
					}
					NewData = NewHandle.GetData();
					
					// Only necessary to do the following for inherited nodes (StackIndex > 0).
					if (NewData && StackIndex > 0)
					{
						// This call creates ICH override templates for the current Blueprint. Without this, the parent node
						// search above can fail when attempting to match an inherited node in the tree via component template.
						NewData->GetObjectForBlueprint(ActorBP);
						for (FSubobjectDataHandle ChildHandle : NewData->GetChildrenHandles())
						{
							FSubobjectData* ChildData = ChildHandle.GetData();
							if (ensure(ChildData != nullptr))
							{
								ChildData->GetObjectForBlueprint(ActorBP);
							}
						}
					}
				}
			}
		}

		// Sort subobjects by type (always put scene components first in the tree)
		OutArray.Sort([](const FSubobjectDataHandle& A, const FSubobjectDataHandle& B)
		{
			return A.GetData()->IsActor() || (A.GetData()->IsSceneComponent() && !B.GetData()->IsActor());
		});
	}
	// Otherwise, this is an actor instance in a level
	else
	{
		// Get the full set of instanced components
		TSet<UActorComponent*> ComponentsToAdd(ActorContext->GetComponents());

		const bool bHideConstructionScriptComponentsInDetailsView = GetDefault<UBlueprintEditorSettings>()->bHideConstructionScriptComponentsInDetailsView;
		auto ShouldAddInstancedActorComponent = [bHideConstructionScriptComponentsInDetailsView](UActorComponent* ActorComp, USceneComponent* ParentSceneComp)
		{
			// Exclude nested DSOs attached to BP-constructed instances, which are not mutable.
			return (ActorComp != nullptr
				&& (!ActorComp->IsVisualizationComponent())
				&& (ActorComp->CreationMethod != EComponentCreationMethod::UserConstructionScript || !bHideConstructionScriptComponentsInDetailsView)
				&& (ParentSceneComp == nullptr || !ParentSceneComp->IsCreatedByConstructionScript() || !ActorComp->HasAnyFlags(RF_DefaultSubObject)))
				&& (ActorComp->CreationMethod != EComponentCreationMethod::Native || FComponentEditorUtils::GetPropertyForEditableNativeComponent(ActorComp));
		};

		// Filter the components by their visibility
		for (TSet<UActorComponent*>::TIterator It(ComponentsToAdd.CreateIterator()); It; ++It)
		{
			UActorComponent* ActorComp = *It;
			USceneComponent* SceneComp = Cast<USceneComponent>(ActorComp);
			USceneComponent* ParentSceneComp = SceneComp != nullptr ? SceneComp->GetAttachParent() : nullptr;
			if (!ShouldAddInstancedActorComponent(ActorComp, ParentSceneComp))
			{
				It.RemoveCurrent();
			}
		}

		TFunction<void(USceneComponent*, FSubobjectDataHandle)> AddInstancedComponentsRecursive = [&](USceneComponent* Component, FSubobjectDataHandle ParentHandle)
		{
			if (Component != nullptr)
			{
				for (USceneComponent* ChildComponent : Component->GetAttachChildren())
				{
					if (ComponentsToAdd.Contains(ChildComponent) && ChildComponent->GetOwner() == Component->GetOwner())
					{
						ComponentsToAdd.Remove(ChildComponent);
						FSubobjectDataHandle NewParentHandle = FactoryCreateSubobjectDataWithParent(ChildComponent, ParentHandle);
						OutArray.Add(NewParentHandle);
						
						AddInstancedComponentsRecursive(ChildComponent, NewParentHandle);
					}
				}
			}
		};

		USceneComponent* RootComponent = ActorContext->GetRootComponent();

		// Add the root component first
		if (RootComponent != nullptr)
		{
			// We want this to be first every time, so remove it from the set of components that will be added later
			ComponentsToAdd.Remove(RootComponent);

			// Add the root component first
			FSubobjectDataHandle RootCompHandle = FactoryCreateSubobjectDataWithParent(RootComponent, RootActorHandle);
			OutArray.Add(RootCompHandle);

			// Recursively add
			AddInstancedComponentsRecursive(RootComponent, RootCompHandle);
		}

		// Sort components by type (always put scene components first in the tree)
		ComponentsToAdd.StableSort([](const UActorComponent& A, const UActorComponent& /* B */)
		{
			return A.IsA<USceneComponent>();
		});

		// Now add any remaining instanced owned components not already added above. This will first add any
		// unattached scene components followed by any instanced non-scene components owned by the Actor instance.
		for (UActorComponent* ActorComp : ComponentsToAdd)
		{
			// Create new subobject data with the original data as their parent.
			OutArray.Add(FactoryCreateSubobjectDataWithParent(ActorComp, RootActorHandle));
		}
	}
}

void USubobjectDataSubsystem::FindAllSubobjectData(FSubobjectData* InData, TSet<FSubobjectData*>& OutVisited) const
{
	if(!InData || OutVisited.Contains(InData))
	{
		return;
	}
	
	OutVisited.Add(InData);
		
	for (FSubobjectDataHandle ChildHandle : InData->GetChildrenHandles())
	{
		FindAllSubobjectData(ChildHandle.GetData(), OutVisited);
	}
}

bool USubobjectDataSubsystem::K2_FindSubobjectDataFromHandle(const FSubobjectDataHandle& Handle, FSubobjectData& OutData) const
{
	const FSubobjectData* Data = Handle.GetData();
	if (Data)
	{
		OutData = *Data;
		return true;
	}
	else
	{
		return false;
	}
}

FSubobjectDataHandle USubobjectDataSubsystem::FindHandleForObject(const FSubobjectDataHandle& Context, const UObject* ObjectToFind, UBlueprint* BPContext /* = nullptr */) const
{
	if(Context.IsValid())
	{
		if(const UActorComponent* ComponentToFind = Cast<UActorComponent>(ObjectToFind))
		{
			// If the given component instance is not already an archetype object
			if(BPContext && !ComponentToFind->IsTemplate())
			{
				// Get the component owner's class object
				check(ComponentToFind->GetOwner() != nullptr);
				UClass* OwnerClass = ComponentToFind->GetOwner()->GetClass();

				// If the given component is one that's created during Blueprint construction
				if (ComponentToFind->IsCreatedByConstructionScript())
				{
					// Check the entire Class hierarchy for the node
					TArray<UBlueprintGeneratedClass*> ParentBPStack;
					UBlueprint::GetBlueprintHierarchyFromClass(OwnerClass, ParentBPStack);

					for(int32 StackIndex = ParentBPStack.Num() - 1; StackIndex >= 0; --StackIndex)
					{
						USimpleConstructionScript* ParentSCS = ParentBPStack[StackIndex] ? ParentBPStack[StackIndex]->SimpleConstructionScript.Get() : nullptr;
						if (ParentSCS)
						{
							// Attempt to locate an SCS node with a variable name that matches the name of the given component
							for (USCS_Node* SCS_Node : ParentSCS->GetAllNodes())
							{
								check(SCS_Node != nullptr);
								if (SCS_Node->GetVariableName() == ComponentToFind->GetFName())
								{
									// We found a match; redirect to the component archetype instance that may be associated with a tree node
									ObjectToFind = SCS_Node->ComponentTemplate;
									break;
								}
							}
						}
					}
				}
				else
				{
					// Get the class default object
					const AActor* CDO = Cast<AActor>(OwnerClass->GetDefaultObject());
					if (CDO)
					{
						// Iterate over the Components array and attempt to find a component with a matching name
						for (UActorComponent* ComponentTemplate : CDO->GetComponents())
						{
							if (ComponentTemplate && ComponentTemplate->GetFName() == ComponentToFind->GetFName())
							{
								// We found a match; redirect to the component archetype instance that may be associated with a tree node
								ObjectToFind = ComponentTemplate;
								break;
							}
						}
					}
				}
			}
		}

		TSet<FSubobjectData*> OutData;
		FindAllSubobjectData(Context.GetSharedDataPtr().Get(), OutData);

		for(FSubobjectData* CurData : OutData)
		{
			if(CurData->GetObject() == ObjectToFind)
			{
				return CurData->GetHandle();
			}
		}
	}
	
	return FSubobjectDataHandle::InvalidHandle;
}

static TSharedPtr<FModuleContextInfo> GetModuleForCppCreation()
{
	TSharedPtr<FModuleContextInfo> OutModuleInfo;

	TArray<TSharedPtr<FModuleContextInfo>> AvailableModules;

	{
		TArray<FModuleContextInfo> CurrentModules = GameProjectUtils::GetCurrentProjectModules();
		check(CurrentModules.Num()); // this should never happen since GetCurrentProjectModules is supposed to add a dummy runtime module if the project currently has no modules

		TArray<FModuleContextInfo> CurrentPluginModules = GameProjectUtils::GetCurrentProjectPluginModules();

		CurrentModules.Append(MoveTemp(CurrentPluginModules));

		AvailableModules.Reserve(CurrentModules.Num());
		for (const FModuleContextInfo& ModuleInfo : CurrentModules)
		{
			AvailableModules.Emplace(MakeShareable(new FModuleContextInfo(ModuleInfo)));
		}
	}

	const FString ProjectName = FApp::GetProjectName();

	// Find initially selected module based on simple fallback in this order..
	// Previously selected module, main project module, a  runtime module
	TSharedPtr<FModuleContextInfo> ProjectModule;
	TSharedPtr<FModuleContextInfo> RuntimeModule;

	for (const auto& AvailableModule : AvailableModules)
	{
		if (AvailableModule->ModuleName == ProjectName)
		{
			ProjectModule = AvailableModule;
		}

		if (AvailableModule->ModuleType == EHostType::Runtime)
		{
			RuntimeModule = AvailableModule;
		}
	}

	if (!OutModuleInfo.IsValid())
	{
		if (ProjectModule.IsValid())
		{
			// use the project module we found
			OutModuleInfo = ProjectModule;
		}
		else if (RuntimeModule.IsValid())
		{
			// use the first runtime module we found
			OutModuleInfo = RuntimeModule;
		}
		else
		{
			// default to just the first module
			OutModuleInfo = AvailableModules[0];
		}
	}

	return OutModuleInfo;
}

UClass* USubobjectDataSubsystem::CreateNewCPPComponent(TSubclassOf<UActorComponent> ComponentClass, const FString& NewClassPath, const FString& NewClassName)
{
	UClass* NewClass = nullptr;

	if (ComponentClass && !NewClassName.IsEmpty() && !NewClassPath.IsEmpty())
	{
		FString HeaderFilePath;
		FString CppFilePath;
		FText FailReason;

		TSharedPtr<FModuleContextInfo> SelectedModuleInfo = GetModuleForCppCreation();
		FNewClassInfo NewClassInfo(ComponentClass);

		// Attempt to add new source files to the project
		const TSet<FString>& DisallowedHeaderNames = FSourceCodeNavigation::GetSourceFileDatabase().GetDisallowedHeaderNames();
		const GameProjectUtils::EAddCodeToProjectResult AddCodeResult = 
			GameProjectUtils::AddCodeToProject(
			NewClassName, 
			NewClassPath, 
			*SelectedModuleInfo, 
			NewClassInfo, 
			DisallowedHeaderNames, 
			/*out*/ HeaderFilePath,
			/*out*/ CppFilePath,
			/*out*/ FailReason
		);

		if (AddCodeResult == GameProjectUtils::EAddCodeToProjectResult::Succeeded)
		{			
			FString AddedClassName = FString::Printf(TEXT("/Script/%s.%s"), *SelectedModuleInfo->ModuleName, *NewClassName);
			NewClass = LoadClass<UActorComponent>(nullptr, *AddedClassName, nullptr, LOAD_None, nullptr);
		}
		else
		{
			UE_LOGF(LogSubobjectSubsystem, Error, "Failed to create a new CPP component: %ls", *FailReason.ToString())
		}
	}

	return NewClass;
}

UClass* USubobjectDataSubsystem::CreateNewBPComponent(TSubclassOf<UActorComponent> ComponentClass, const FString& NewClassPath, const FString& NewClassName)
{
	UClass* NewClass = nullptr;

	if(ComponentClass)
	{
		// Ensure that the class is a blueprint type
		// Add logging to make it easier to debug what is going on if you are calling this function from a Blueprint
		if(!FKismetEditorUtilities::CanCreateBlueprintOfClass(ComponentClass))
		{
			UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to Create new BP Component: '%ls' is not a blueprintable class!", *ComponentClass->GetDisplayNameText().ToString());
		}
		else if (NewClassName.IsEmpty())
		{
			UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to Create new BP Component: An NewClassName was given!");
		}
		else if (NewClassPath.IsEmpty())
		{
			UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to Create new BP Component: An invalid class path was given!");
		}
		else
		{
			const FString PackagePath = NewClassPath / NewClassName;

			// Check for an existing object
			if(UObject* ExistingObject = StaticFindObject(UObject::StaticClass(), nullptr, *PackagePath))
			{
				UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to Create new BP Component: A class with a name '%ls' already exists!", *PackagePath);
				return nullptr;
			}
		
			if (UPackage* Package = CreatePackage(*PackagePath))
			{
				// Create and init a new Blueprint			
				if (UBlueprint* NewBP = FKismetEditorUtilities::CreateBlueprint(ComponentClass, Package, FName(*NewClassName), BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass()))
				{
					// Notify the asset registry
					FAssetRegistryModule::AssetCreated(NewBP);

					// Mark the package dirty...
					Package->MarkPackageDirty();
					NewClass = NewBP->GeneratedClass;
				}
			}
		}
	}
	else
	{
		UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to Create new BP Component: ComponentClass is required!");
	}
	
	return NewClass;
}

static void SaveSCSCurrentState(USimpleConstructionScript* SCSObj)
{
	if (SCSObj)
	{
		SCSObj->SaveToTransactionBuffer();
	}
}

static void ConformTransformRelativeToParent(USceneComponent* SceneComponentTemplate, const USceneComponent* ParentSceneComponent)
{
	// If we find a match, calculate its new position relative to the scene root component instance in its current scene
	FTransform ComponentToWorld(SceneComponentTemplate->GetRelativeRotation(), SceneComponentTemplate->GetRelativeLocation(), SceneComponentTemplate->GetRelativeScale3D());
	FTransform ParentToWorld = (SceneComponentTemplate->GetAttachSocketName() != NAME_None) ? ParentSceneComponent->GetSocketTransform(SceneComponentTemplate->GetAttachSocketName(), RTS_World) : ParentSceneComponent->GetComponentToWorld();
	FTransform RelativeTM = ComponentToWorld.GetRelativeTransform(ParentToWorld);

	// Store new relative location value (if not set to absolute)
	if (!SceneComponentTemplate->IsUsingAbsoluteLocation())
	{
		SceneComponentTemplate->SetRelativeLocation_Direct(RelativeTM.GetTranslation());
	}

	// Store new relative rotation value (if not set to absolute)
	if (!SceneComponentTemplate->IsUsingAbsoluteRotation())
	{
		FRotator RelativeRotation = RelativeTM.Rotator();

		// Identity quaternion can convert to a zero rotator with a -0.0 Roll value, correct that
		if (RelativeRotation == FRotator{ 0.0, 0.0, -0.0 })
		{
			RelativeRotation = FRotator{ 0.0 };
		}
		
		SceneComponentTemplate->SetRelativeRotation_Direct(RelativeRotation);
	}

	// Store new relative scale value (if not set to absolute)
	if (!SceneComponentTemplate->IsUsingAbsoluteScale())
	{
		SceneComponentTemplate->SetRelativeScale3D_Direct(RelativeTM.GetScale3D());
	}
}

FSubobjectDataHandle USubobjectDataSubsystem::FindSceneRootForSubobject(const FSubobjectDataHandle& InHandle) const
{
	if(!InHandle.IsValid())
	{
		return FSubobjectDataHandle::InvalidHandle;
	}

	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_FindSceneRootForSubobject(InHandle))
		{
			return Entry.Adapter->FindSceneRootForSubobject(InHandle);
		}
	}
	
	FSubobjectDataHandle ActorHandle = InHandle;
	FSubobjectData* ActorData = ActorHandle.GetData();
	
	// If the given handle is not an actor, then we can walk it's parent chain back up until we find it
	while (ActorData && !ActorData->IsActor())
	{
		ActorHandle = ActorData->GetParentHandle();
		ActorData = ActorHandle.GetData();
	}

	// If the given handle is an actor, then we have to use it's first scene component
	if(ensure(ActorData && ActorData->IsActor()))
	{
		TArray<FSubobjectDataHandle> ChildHandles = ActorData->GetChildrenHandles();
		for(const FSubobjectDataHandle& ChildHandle : ChildHandles)
		{
			FSubobjectData* ChildData = ChildHandle.GetData();
			if(ChildData && ChildData->IsDefaultSceneRoot())
			{
				return ChildHandle;
			}
		}
	}

	// Last resort is to just return the last known actor handle
	return ActorHandle;
}

FSubobjectDataHandle USubobjectDataSubsystem::FindParentForNewSubobject(const UObject* NewSubobject, const FSubobjectDataHandle& SelectedParent)
{
	FSubobjectDataHandle TargetParentHandle = SelectedParent;
	check(TargetParentHandle.IsValid());
	FSubobjectData* TargetParentData = TargetParentHandle.GetData();

	// If the current selection belongs to a child actor template, move the target to its outer component node.
	while(TargetParentHandle.IsValid() && TargetParentData->IsChildActorSubtreeObject())
	{
		TargetParentHandle = TargetParentData->GetParentHandle();
		TargetParentData = TargetParentHandle.GetData();
		check(TargetParentData);
	}
	
	if(const USceneComponent* NewSceneComponent = Cast<USceneComponent>(NewSubobject))
	{
		if(TargetParentData)
		{
			if (TargetParentData->IsActor())
			{
				AActor* TargetActor = TargetParentData->GetMutableObject<AActor>();
				check(TargetActor);
				USceneComponent* TargetRootComp = TargetActor ? TargetActor->GetDefaultAttachComponent() : nullptr;
				// A component should attach to the target scene component 
				FSubobjectDataHandle RootComponentHandle = FactoryCreateSubobjectDataWithParent(
					TargetRootComp ? Cast<UObject>(TargetRootComp) : Cast<UObject>(TargetActor),
					TargetParentData->GetHandle()
				);
				
				if (RootComponentHandle.IsValid())
				{
					TargetParentHandle = RootComponentHandle;
					const USceneComponent* CastTargetToSceneComponent = Cast<USceneComponent>(TargetParentData->GetComponentTemplate());
					if (CastTargetToSceneComponent == nullptr || !NewSceneComponent->CanAttachAsChild(CastTargetToSceneComponent, NAME_None))
					{
						TargetParentHandle = FindSceneRootForSubobject(SelectedParent); // Default to SceneRoot
					}
				}
			}
			else if(TargetParentData->IsComponent())
			{
				const USceneComponent* CastTargetToSceneComponent = Cast<USceneComponent>(TargetParentData->GetComponentTemplate());
				if (CastTargetToSceneComponent == nullptr || !NewSceneComponent->CanAttachAsChild(CastTargetToSceneComponent, NAME_None))
				{
					TargetParentHandle = FindSceneRootForSubobject(SelectedParent); // Default to SceneRoot
				}
			}
		}
		else
		{
			TargetParentHandle = FindSceneRootForSubobject(SelectedParent);
		}
	}
	else
	{
		// Non-scene components should be parented to the base actor in the hierarchy
		TargetParentHandle = GetActorRootHandle(SelectedParent);
	}
	
	return TargetParentHandle;
}

FSubobjectDataHandle USubobjectDataSubsystem::GetActorRootHandle(const FSubobjectDataHandle& StartingHandle)
{
	FSubobjectDataHandle CurrentHandle = StartingHandle;
	FSubobjectData* CurrentData = CurrentHandle.GetData();

	TSet<FSubobjectDataHandle> VisitedHandles;
	VisitedHandles.Add(StartingHandle);
	
	while (CurrentData && !CurrentData->IsActor())
	{
		CurrentHandle = CurrentData->GetParentHandle();
		CurrentData = CurrentHandle.GetData();

		// If we have already visited this node, then exit out of this loop
		if (VisitedHandles.Contains(CurrentHandle))
		{
			const UObject* CurrentObj = CurrentData ? CurrentData->GetObject() : nullptr;						

			ensureMsgf(
				false, 
				TEXT("[%hs] Duplicate visits when getting that actor's root handle! Exiting to prevent an infinite loop. Duplicate object: '%s' Flags: %s"), 
				__func__,
				*GetNameSafe(CurrentObj),
				*LexToString(CurrentObj ? CurrentObj->GetFlags() : RF_NoFlags));

			break;
		}

		VisitedHandles.Add(CurrentHandle);
	}

	return CurrentHandle;
}

FSubobjectDataHandle USubobjectDataSubsystem::AddNewSubobject(const FAddNewSubobjectParams& Params, FText& FailReason)
{
	FSubobjectDataHandle NewDataHandle = FSubobjectDataHandle::InvalidHandle;
	// Check for valid parent
	if (!Params.ParentHandle.IsValid() || !Params.NewClass)
	{
		return NewDataHandle;
	}

	UClass* NewClass = Params.NewClass;
	UObject* Asset = Params.AssetOverride;
	const FSubobjectDataHandle& ParentObjHandle = Params.ParentHandle;

	if (NewClass->ClassWithin && NewClass->ClassWithin != UObject::StaticClass())
	{
		FailReason =  LOCTEXT("AddComponentFailed", "Cannot add components that have \"Within\" markup");
		return NewDataHandle;
	}

	// Ensure that the new class is actually a valid subobject type.
	// As of right now, that means the class has to be a child of an Actor Component. 
	if (!NewClass->IsChildOf(UActorComponent::StaticClass()))
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ClassName"), NewClass->GetDisplayNameText());
		FailReason = FText::Format(LOCTEXT("AddComponentFailed_InvalidClassType", "Cannot add a subobject of class '{ClassName}'"), Arguments);
		return NewDataHandle;
	}
	
	FName TemplateVariableName;
	const USCS_Node* SCSNode = Cast<const USCS_Node>(Asset);
	UActorComponent* ComponentTemplate = (SCSNode ? ToRawPtr(SCSNode->ComponentTemplate) : Cast<UActorComponent>(Asset));

	if (SCSNode)
	{
		TemplateVariableName = SCSNode->GetVariableName();
		Asset = nullptr;
	}
	else if (ComponentTemplate)
	{
		Asset = nullptr;
	}

	// We check for an invalid parent handle above, so we assume it is valid here.
	check(Params.ParentHandle.IsValid());

	// Editor utility scripts may set the parent handle but not a Blueprint context. It is arguable whether this constitutes a valid use case; although
	// for backwards compatibility we'll continue to support it. However, in both the BP and instance in-editor use cases, data nodes that correlate to
	// an SCS node will use the SCS template data, which means that we'll find a valid Blueprint context in that case. For editor scripting, we want it
	// to always modify the Blueprint, but for "normal" editor operations (e.g. drag-and-drop), we need to bypass this if the root is an Actor instance.
	UBlueprint* Blueprint = Params.BlueprintContext;
	const FSubobjectDataHandle& ParentActorRootHandle = GetActorRootHandle(ParentObjHandle);
	if (!Blueprint)
	{
		// maybe the parent handle has a bp for context:
		Blueprint = ParentObjHandle.GetData()->GetBlueprintBeingEdited();
	}
	
	if(Blueprint)
	{
		if (Blueprint->SimpleConstructionScript == nullptr)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("BlueprintName"), FText::FromString(Blueprint->GetName()));
			FailReason = FText::Format(LOCTEXT("AddComponentFailed_NoSCS",
				"Cannot add a component to Blueprint '{BlueprintName}' because it has no construction script "
				"(it may be a const, level-script, interface, macro, or function-library Blueprint)."), Arguments);
			return NewDataHandle;
		}
		Blueprint->Modify();
		SaveSCSCurrentState(Blueprint->SimpleConstructionScript);
		UActorComponent* NewComponent = nullptr;
		
		// Defer Blueprint class regeneration and tree updates until after we copy any object properties from a source template.
        const bool bMarkBlueprintModified = false;
		
		FName NewVariableName;
		if (ComponentTemplate)
		{
			if (!TemplateVariableName.IsNone())
			{
				NewVariableName = TemplateVariableName;
			}
			else
			{
				FString TemplateName = ComponentTemplate->GetName();
				NewVariableName = (TemplateName.EndsWith(USimpleConstructionScript::ComponentTemplateNameSuffix) 
                                    ? FName(*TemplateName.LeftChop(USimpleConstructionScript::ComponentTemplateNameSuffix.Len()))
                                    : ComponentTemplate->GetFName());
			}
		}
		else if (Asset)
		{
			NewVariableName = *FComponentEditorUtils::GenerateValidVariableNameFromAsset(Asset, nullptr);
		}

		USCS_Node* NewSCSNode = Blueprint->SimpleConstructionScript->CreateNode(NewClass, NewVariableName);
		NewSCSNode->Modify();
		NewComponent = NewSCSNode->ComponentTemplate;
		// stuff from AddNewNode
		// Assign this new node to an override asset if there is one
		if(Asset)
		{
			FComponentAssetBrokerage::AssignAssetToComponent(NewComponent, Asset);
		}

		FSubobjectDataHandle TargetAttachmentHandle = FindParentForNewSubobject(NewComponent, ParentObjHandle);
		FSubobjectData* TargetAttachment = TargetAttachmentHandle.GetData();
		check(TargetAttachment);
		
		// Create a new subobject data set with this component. Use the SCS node here and the subobject data
		// will correctly associate the component template
		NewDataHandle = FactoryCreateSubobjectDataWithParent(NewSCSNode, TargetAttachment->GetHandle());

		// Actually add this new node to the SimpleConstructionScript on the blueprint
		AttachSubobject(TargetAttachment->GetHandle(), NewDataHandle);
		
		// Potentially adjust variable names for any child blueprints
        const FName VariableName = NewSCSNode->GetVariableName();
        if(VariableName != NAME_None)
        {
        	FBlueprintEditorUtils::ValidateBlueprintChildVariables(Blueprint, VariableName);
        }
		
		if (ComponentTemplate)
		{
			UEngine::CopyPropertiesForUnrelatedObjects(ComponentTemplate, NewComponent);
			NewComponent->UpdateComponentToWorld();
		}

		if (Params.bConformTransformToParent)
		{
			if (USceneComponent* AsSceneComp = Cast<USceneComponent>(NewComponent))
			{
				// This problem because it is using the generated bp class as a parent
				if (const USceneComponent* ParentSceneComp = CastChecked<USceneComponent>(TargetAttachment->GetComponentTemplate(), ECastCheckedType::NullAllowed))
				{
					ConformTransformRelativeToParent(AsSceneComp, ParentSceneComp);
				}
			}
		}

		// Wait until here to mark as structurally modified because we don't want any RerunConstructionScript() calls
		// to happen until AFTER we've serialized properties from the source object.
		if (!Params.bSkipMarkBlueprintModified)
		{
			FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
		}
	}
	else	// Not in a BP context
	{
		FSubobjectData* ParentObjData = ParentObjHandle.GetData();

		check(ParentObjData);
		
		// If this is a component template, then we can just duplicate it
		if(ComponentTemplate)
		{
			UActorComponent* NewComponent = FComponentEditorUtils::DuplicateComponent(ComponentTemplate);
			// Create a new subobject data set with this component
			NewDataHandle = FactoryCreateSubobjectDataWithParent(NewComponent, ParentObjData->GetHandle());
		}
		else
		{
			AActor* ActorInstance = ParentObjData->GetMutableActorContext();

			// If we can't find an actor instance then search up the hierarchy for one that we can use instead
			if (!ActorInstance)
			{
				const FSubobjectDataHandle& ActorRootHandle = GetActorRootHandle(ParentObjData->GetHandle());
				if (FSubobjectData* ActorInstanceData = ActorRootHandle.GetData())
				{
					ActorInstance = ActorInstanceData->GetMutableActorContext();
				}
			}

			if (ActorInstance)
			{
				ActorInstance->Modify();

				// Create an appropriate name for the new component
				FName NewComponentName = NAME_None;
				if (Asset)
				{
					NewComponentName = *FComponentEditorUtils::GenerateValidVariableNameFromAsset(Asset, ActorInstance);
				}
				else
				{
					NewComponentName = *FComponentEditorUtils::GenerateValidVariableName(NewClass, ActorInstance);
				}

				// Get the set of owned components that exists prior to instancing the new component.
				TInlineComponentArray<UActorComponent*> PreInstanceComponents;
				ActorInstance->GetComponents(PreInstanceComponents);

				// Construct the new component and attach as needed
				UActorComponent* NewInstanceComponent = NewObject<UActorComponent>(ActorInstance, NewClass, NewComponentName, RF_Transactional);

				// Do Scene Attachment if this new Component is a USceneComponent
				if (USceneComponent* NewSceneComponent = Cast<USceneComponent>(NewInstanceComponent))
				{
					// If this is an actor with no subobjects on it, then set the new subobject as scene root of the actor.
					// This can occur if the user has placed in a native C++ class to the world with no subobjects on it
					if ((ParentObjData->IsActor() && ParentObjData->GetChildrenHandles().IsEmpty()) || 
						ActorInstance->GetRootComponent() == nullptr)
					{
						ActorInstance->SetRootComponent(NewSceneComponent);
					}
					else
					{
						USceneComponent* AttachTo = Cast<USceneComponent>(ParentObjData->GetMutableComponentTemplate());
						if (AttachTo && AttachTo->IsTemplate())
						{
							UActorComponent** ArchetypeAttachment = PreInstanceComponents.FindByPredicate([AttachTo](const UActorComponent* A)
							{ 
								return A && A->GetArchetype() == AttachTo;
							});

							AttachTo = ArchetypeAttachment ? CastChecked<USceneComponent>(*ArchetypeAttachment) : nullptr;
						}

						if (AttachTo == nullptr)
						{
							AttachTo = ActorInstance->GetRootComponent();
						}
						check(AttachTo != nullptr);

						// Make sure that the mobility of the new scene component is such that we can attach it
						if (AttachTo->Mobility == EComponentMobility::Movable)
						{
							NewSceneComponent->Mobility = EComponentMobility::Movable;
						}
						else if (AttachTo->Mobility == EComponentMobility::Stationary && NewSceneComponent->Mobility == EComponentMobility::Static)
						{
							NewSceneComponent->Mobility = EComponentMobility::Stationary;
						}

						NewSceneComponent->AttachToComponent(AttachTo, FAttachmentTransformRules::SnapToTargetNotIncludingScale);
					}
				}

				// If the component was created from/for a particular asset, assign it now
				if (Asset)
				{
					FComponentAssetBrokerage::AssignAssetToComponent(NewInstanceComponent, Asset);
				}

				// Add to SerializedComponents array so it gets saved
				ActorInstance->AddInstanceComponent(NewInstanceComponent);
				NewInstanceComponent->OnComponentCreated();
				NewInstanceComponent->RegisterComponent();

				// Register any new components that may have been created during construction of the instanced component, but were not explicitly registered.
				TInlineComponentArray<UActorComponent*> PostInstanceComponents;
				ActorInstance->GetComponents(PostInstanceComponents);
				for (UActorComponent* ActorComponent : PostInstanceComponents)
				{
					if (!ActorComponent->IsRegistered() && ActorComponent->bAutoRegister && IsValidChecked(ActorComponent) && !PreInstanceComponents.Contains(ActorComponent))
					{
						ActorComponent->RegisterComponent();
					}
				}

				// Rerun construction scripts
				ActorInstance->RerunConstructionScripts();

				// If the running the construction script destroyed the new node, don't create an entry for it
				if (IsValidChecked(NewInstanceComponent))
				{
					// Create a new subobject data set with this component
					NewDataHandle = FactoryCreateSubobjectDataWithParent(NewInstanceComponent, ParentObjData->GetHandle());
				}
				
				BroadcastInstanceChanges();
			}
			else
			{
				FailReason = LOCTEXT("AddComponentFailed_Inherited", "Cannot add components within an Inherited hierarchy");
			}
		}
	}

	// If the new handle is valid, broadcast to any listeners that a new subobject was added.
	if (const FSubobjectData* NewData = NewDataHandle.GetData())
	{
		OnNewSubobjectAdded_Delegate.Broadcast(*NewData);
	}
	
	return NewDataHandle;
}

int32 USubobjectDataSubsystem::DeleteSubobjects(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete, FSubobjectDataHandle& OutComponentToSelect, UBlueprint* BPContext/* = nullptr*/, bool bForce /* = false */)
{
	int32 NumDeletedSubobjects = 0;

	if(!ContextHandle.IsValid() || SubobjectsToDelete.Num() == 0)
	{
		return NumDeletedSubobjects;
	}
	const FSubobjectData* SubobjectData = ContextHandle.GetData();
	if (!SubobjectData)
	{
		return NumDeletedSubobjects;
	}

	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_DeleteSubobjects(ContextHandle, SubobjectsToDelete, BPContext, bForce))
		{
			NumDeletedSubobjects = Entry.Adapter->DeleteSubobjects(
				ContextHandle,
				SubobjectsToDelete);
			return NumDeletedSubobjects;
		}
	}
	
	UObject* ContextObj = SubobjectData->GetMutableObject();
	check(ContextObj);

	ContextObj->Modify();
	
	if (BPContext)
	{
		for (const FSubobjectDataHandle& Handle : SubobjectsToDelete)
		{
			if (Handle.IsValid())
			{
				if (const FSubobjectData* Data = Handle.GetData())
				{
					if (!Data->CanDelete() && !bForce)
					{
						UE_LOGF(LogSubobjectSubsystem, Warning, "Cannot delete subobject '%ls' from '%ls'!", *Data->GetDisplayString(), *BPContext->GetFullName());
						continue;
					}

					USCS_Node* SCS_Node = Data->GetSCSNode();
					
					if (SCS_Node)
					{
						USimpleConstructionScript* SCS = SCS_Node->GetSCS();						
						BPContext->Modify();
						SaveSCSCurrentState(SCS);
					
						FBlueprintEditorUtils::RemoveVariableNodes(BPContext, Data->GetVariableName());

						// If there are any Bound Component events for this property then give them compiler errors
						TArray<UK2Node_ComponentBoundEvent*> EventNodes;
						FKismetEditorUtilities::FindAllBoundEventsForComponent(BPContext, SCS_Node->GetVariableName(), EventNodes);
						if (EventNodes.Num() > 0)
						{
							// Find any dynamic delegate nodes and give a compiler error for each that is problematic
							FCompilerResultsLog LogResults;
							FMessageLog MessageLog("BlueprintLog");

							// Add a compiler error for each bound event node
							for (UK2Node_ComponentBoundEvent* Node : EventNodes)
							{
								LogResults.Error(*LOCTEXT("RemoveBoundEvent_Error", "The component that @@ was bound to has been deleted! This node is no longer valid").ToString(), Node);
							}

							// Notify the user that these nodes are no longer valid
							MessageLog.NewPage(LOCTEXT("RemoveBoundEvent_Error_Label", "Removed Owner of Component Bound Event"));
							MessageLog.AddMessages(LogResults.Messages);
							MessageLog.Notify(LOCTEXT("RemoveBoundEvent_Error_Msg", "Removed Owner of Component Bound Event"));

							// Focus on the first node that we found
							FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(EventNodes[0]);
						}

						// Remove node from SCS tree
						SCS->RemoveNodeAndPromoteChildren(SCS_Node);
						++NumDeletedSubobjects;
						
						// Clear the delegate
						SCS_Node->SetOnNameChanged(FSCSNodeNameChanged());

						// on removal, since we don't move the template from the GeneratedClass (which we shouldn't, as it would create a 
						// discrepancy with existing instances), we rename it instead so that we can re-use the name without having to compile  
						// (we still have a problem if they attempt to name it to what ever we choose here, but that is unlikely)
						// note: skip this for the default scene root; we don't actually destroy that node when it's removed, so we don't need the template to be renamed.
						if (!Data->IsDefaultSceneRoot() && SCS_Node->ComponentTemplate != nullptr)
						{
							const FName TemplateName = SCS_Node->ComponentTemplate->GetFName();
							const FString RemovedName = SCS_Node->GetVariableName().ToString() + TEXT("_REMOVED_") + FGuid::NewGuid().ToString();

							SCS_Node->ComponentTemplate->Modify();
							SCS_Node->ComponentTemplate->Rename(*RemovedName, /*NewOuter =*/nullptr, REN_DontCreateRedirectors);

							TArray<UObject*> ArchetypeInstances;
							auto DestroyArchetypeInstances = [&ArchetypeInstances, &RemovedName](UActorComponent* ComponentTemplate)
							{
								ComponentTemplate->GetArchetypeInstances(ArchetypeInstances);
								for (UObject* ArchetypeInstance : ArchetypeInstances)
								{
									if (!ArchetypeInstance->HasAllFlags(RF_ArchetypeObject | RF_InheritableComponentTemplate))
									{
										CastChecked<UActorComponent>(ArchetypeInstance)->DestroyComponent();
										ArchetypeInstance->Rename(*RemovedName, nullptr, REN_DontCreateRedirectors);
									}
								}
							};

							DestroyArchetypeInstances(SCS_Node->ComponentTemplate);

							if (BPContext)
							{
								// Children need to have their inherited component template instance of the component renamed out of the way as well
								TArray<UClass*> ChildrenOfClass;
								GetDerivedClasses(BPContext->GeneratedClass, ChildrenOfClass);

								for (UClass* ChildClass : ChildrenOfClass)
								{
									UBlueprintGeneratedClass* BPChildClass = CastChecked<UBlueprintGeneratedClass>(ChildClass);

									if (UActorComponent* Component = (UActorComponent*)FindObjectWithOuter(BPChildClass, UActorComponent::StaticClass(), TemplateName))
									{
										Component->Modify();
										Component->Rename(*RemovedName, /*NewOuter =*/nullptr, REN_DontCreateRedirectors);

										DestroyArchetypeInstances(Component);
									}
								}
							}
						}
						else if(!bForce)
						{
							UE_LOGF(LogSubobjectSubsystem, Warning, "Cannot remove subobject '%ls' because it is the default scene root!", *Data->GetDisplayString());
						}

						if (Data->IsDefaultSceneRoot())
						{
							// The DefaultSceneRoot is never deleted, but just hidden instead.
							// We reset its name, so that it can be reused by other Components.
							SCS_Node->SetVariableName(USceneComponent::GetDefaultSceneRootVariableName());
						}
					}
				}
			}
		}

		// Will call UpdateTree as part of OnBlueprintChanged handling
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BPContext);
	}
	else	// This is an actor instance
	{
		TArray<UActorComponent*> ComponentsToDelete;
		for(const FSubobjectDataHandle& Handle : SubobjectsToDelete)
		{
			if(Handle.IsValid())
			{
				if(const FSubobjectData* Data = Handle.GetData())
				{
					if (!Data->CanDelete())
					{
						UE_LOGF(LogSubobjectSubsystem, Warning, "Cannot delete subobject '%ls' from '%ls'!", *Data->GetDisplayString(), *ContextObj->GetFullName());
						continue;
					}

					if(UActorComponent* Template = Data->GetMutableComponentTemplate())
					{
						ComponentsToDelete.Add(Template);
					}					
				}		
			}
		}

		// Actually delete the components if we have any that can be
		if (!ComponentsToDelete.IsEmpty())
		{
			UActorComponent* ActorComponentToSelect = nullptr;
			NumDeletedSubobjects = FComponentEditorUtils::DeleteComponents(ComponentsToDelete, ActorComponentToSelect);
			if (ActorComponentToSelect)
			{
				OutComponentToSelect = FindHandleForObject(ContextHandle, ActorComponentToSelect);
			}
			BroadcastInstanceChanges();
		}
	}

	return NumDeletedSubobjects;
}

int32 USubobjectDataSubsystem::DeleteSubobjects(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete, UBlueprint* BPContext /*= nullptr*/)
{
	FSubobjectDataHandle Dummy;
	return DeleteSubobjects(ContextHandle, SubobjectsToDelete, Dummy, BPContext);
}

int32 USubobjectDataSubsystem::K2_DeleteSubobjectFromInstance(const FSubobjectDataHandle& ContextHandle, const FSubobjectDataHandle& SubobjectToDelete)
{
	return DeleteSubobject(ContextHandle, SubobjectToDelete);
}

int32 USubobjectDataSubsystem::K2_DeleteSubobjectsFromInstance(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete)
{
	return DeleteSubobjects(ContextHandle, SubobjectsToDelete);
}

int32 USubobjectDataSubsystem::DeleteSubobject(const FSubobjectDataHandle& ContextHandle, const FSubobjectDataHandle& SubobjectToDelete, UBlueprint* BPContext)
{
	TArray<FSubobjectDataHandle> Handles { SubobjectToDelete };
	return DeleteSubobjects(ContextHandle, Handles, BPContext);
}

bool USubobjectDataSubsystem::RenameSubobject(const FSubobjectDataHandle& Handle, const FText& InNewName)
{
	FText OutErrorMessage;
	if (!IsValidRename(Handle, InNewName, OutErrorMessage))
	{
		return false;	
	}

	const FSubobjectData* Data = Handle.GetData();
	if (!Data)
	{
		return false;
	}

	// For root actor nodes
	if (AActor* Actor = Data->GetMutableObject<AActor>())
	{
		if (Actor->IsActorLabelEditable() && !InNewName.ToString().Equals(Actor->GetActorLabel(), ESearchCase::CaseSensitive))
		{
			const FScopedTransaction Transaction(LOCTEXT("SCSEditorRenameActorTransaction", "Rename Actor"));
			FActorLabelUtilities::RenameExistingActor(Actor, InNewName.ToString());
			return true;
		}
	}
	
	// For components: either instanced or simple construction script ("Add Component" in Blueprint editor)
	if (UActorComponent* ComponentInstance = Data->GetMutableComponentTemplate())
	{
		UBlueprint* BP = Data->GetBlueprint();
		const FString DesiredName = InNewName.ToString();
		auto ValidateNameInBP = [BP](const FString& DesiredName) 
		{
			return FKismetNameValidator(BP).IsValid(DesiredName) == EValidatorResult::Ok
				? FName(DesiredName)
				: FBlueprintEditorUtils::FindUniqueKismetName(BP, DesiredName);
		};
		
		// When placed in level, just the component needs to be renamed.
		if (Data->IsInstancedComponent())
		{
			// UBlueprint instance required may not always be available, e.g. if we added a non-Blueprint C++ class into the level like AStaticMeshActor.
			// If they are available, use FKismetNameValidator and otherwise we'll just do our best here and replace invalid characters.
			const FString NewNameAsString = BP
				? ValidateNameInBP(DesiredName).ToString()
				: FBlueprintEditorUtils::ReplaceInvalidBlueprintNameCharactersInline(DesiredName);
			
			// Name collision could occur due to e.g. our archetype being updated and causing a conflict with our ComponentInstance:
			if (StaticFindObject(UObject::StaticClass(), ComponentInstance->GetOuter(), *NewNameAsString) == nullptr)
			{
				const ERenameFlags RenameFlags = REN_DontCreateRedirectors;
				ComponentInstance->Rename(*NewNameAsString, nullptr, RenameFlags);
			}
			
			return true;
		}

		// In Blueprint editor, the component variable name must be updated.
		if (BP)
		{
			// Is this desired name the same as what is already there? If so then don't bother.
			USCS_Node* SCSNode = Data->GetSCSNode();
			if(SCSNode && SCSNode->GetVariableName().ToString().Equals(DesiredName))
			{
				return true;
			}
			
			const FName ValidatedNewName = ValidateNameInBP(DesiredName);
			FBlueprintEditorUtils::RenameComponentMemberVariable(BP, SCSNode, ValidatedNewName);
			return true;
		}
	}
	
	return false;
}

bool USubobjectDataSubsystem::ChangeSubobjectClass(const FSubobjectDataHandle& Handle, const UClass* NewClass)
{
	if (!GetAllowNativeComponentClassOverrides())
	{
		return false;
	}

	const FSubobjectData* Data = Handle.GetData();
	if (Data && Data->IsNativeComponent())
	{
		// For instanced components
		if (UActorComponent* ComponentTemplate = Data->GetMutableComponentTemplate())
		{
			if (UBlueprint* BlueprintObj = Data->GetBlueprint())
			{
				UClass* BaseClass = ComponentTemplate->GetClass();

				if (const FBPComponentClassOverride* Override = BlueprintObj->ComponentClassOverrides.FindByKey(ComponentTemplate->GetFName()))
				{
					AActor* Owner = ComponentTemplate->GetOwner();
					AActor* OwnerArchetype = CastChecked<AActor>(Owner->GetArchetype());
					if (UActorComponent* ArchetypeComponent = Cast<UActorComponent>((UObject*)FindObjectWithOuter(OwnerArchetype, UActorComponent::StaticClass(), ComponentTemplate->GetFName())))
					{
						BaseClass = ArchetypeComponent->GetClass();
					}
				}

				if (!NewClass->IsChildOf(BaseClass))
				{
					return false;
				}

				const FScopedTransaction Transaction(LOCTEXT("SetComponentClassOverride", "Set Component Class Override"));

				const FName ComponentTemplateName = ComponentTemplate->GetFName();

				BlueprintObj->Modify();

				if (FBPComponentClassOverride* Override = BlueprintObj->ComponentClassOverrides.FindByKey(ComponentTemplateName))
				{
					bool bRemoveEntry = false;
					bool bFoundOverride = false;

					UBlueprint* ParentBP = UBlueprint::GetBlueprintFromClass(Cast<UBlueprintGeneratedClass>(BlueprintObj->ParentClass));
					while (ParentBP)
					{
						if (FBPComponentClassOverride* ParentOverride = BlueprintObj->ComponentClassOverrides.FindByKey(ComponentTemplateName))
						{
							bRemoveEntry = (ParentOverride->ComponentClass == NewClass);
							bFoundOverride = true;
							break;
						}
					}
					if (!bFoundOverride)
					{
						bRemoveEntry = (ComponentTemplate->GetClass() == NewClass);
					}
					if (bRemoveEntry)
					{
						BlueprintObj->ComponentClassOverrides.RemoveAllSwap([ComponentTemplateName](const FBPComponentClassOverride& CCOverride) { return (CCOverride.ComponentName == ComponentTemplateName); });
						if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(BlueprintObj->GeneratedClass))
						{
							Class->Modify();
							Class->ComponentClassOverrides.RemoveAllSwap([ComponentTemplateName](const FBPComponentClassOverride& CCOverride) { return (CCOverride.ComponentName == ComponentTemplateName); });
						}
					}
					else
					{
						Override->ComponentClass = NewClass;
					}
				}
				else
				{
					BlueprintObj->ComponentClassOverrides.Emplace(FBPComponentClassOverride(ComponentTemplateName, NewClass));
					if (UBlueprintGeneratedClass* Class = Cast<UBlueprintGeneratedClass>(BlueprintObj->GeneratedClass))
					{
						Class->Modify();
						Class->ComponentClassOverrides.Emplace(FBPComponentClassOverride(ComponentTemplateName, NewClass));
					}
				}

				// Custom transaction change that operates on the UBlueprint and replaces all instances of the subobject with one of the new class
				struct FSubobjectClassChange : public FCommandChange
				{
					FSubobjectClassChange(FGuid InOperationGuid, FName InTemplateName, const UClass* InApplyClass, const UClass* InRevertClass)
						: OperationGuid(InOperationGuid)
						, TemplateName(InTemplateName)
						, ApplyClass(InApplyClass)
						, RevertClass(InRevertClass)
					{
					}

					// Utility function to assemble the name of the object in the transient package to use as template from replacement
					static FString MakeReplacementName(const FString& CurrentName, UObject* SubobjectOuter, const TCHAR* ReplaceName, FGuid OperationGuid)
					{
						return FString::Printf(TEXT("%s_%s_%s_%d"), *CurrentName, ReplaceName, *OperationGuid.ToString(), SubobjectOuter->GetUniqueID());
					}

					// Swaps out all of the supplied subobject instances with objects of the new class
					void SwapObjects(TArrayView<UObject*> SubobjectInstances, const TCHAR* ReplaceName, const UClass* NewClass, TArrayView<UObject*> ReplacementTemplates = TArrayView<UObject*>())
					{					
						TMap<UObject*, UObject*> ReferenceReplacementMap;
						const FString TemplateNameStr = TemplateName.ToString();

						for (int32 ObjectIndex = 0; ObjectIndex < SubobjectInstances.Num(); ++ObjectIndex)
						{
							UObject* Subobject = SubobjectInstances[ObjectIndex];
							UObject* SubobjectOuter = Subobject->GetOuter();
							const EObjectFlags SubobjectFlags = Subobject->GetFlags();

							bool bWasRegistered = false;
							if (UActorComponent* Comp = Cast<UActorComponent>(Subobject))
							{
								bWasRegistered = Comp->IsRegistered();
								if (bWasRegistered)
								{
									Comp->UnregisterComponent();
								}
							}

							Subobject->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
							Subobject->MarkAsGarbage();

							UObject* NewInstance = nullptr;
							UObject* ReplaceTemplate = nullptr;
							if (ReplacementTemplates.Num() > 0)
							{
								ReplaceTemplate = ReplacementTemplates[ObjectIndex];
							}
							else
							{
								ReplaceTemplate = static_cast<UObject*>(FindObjectWithOuter(GetTransientPackage(), nullptr, *MakeReplacementName(TemplateNameStr, SubobjectOuter, ReplaceName, OperationGuid)));
							}

							if (ReplaceTemplate)
							{ 
								NewInstance = StaticDuplicateObject(ReplaceTemplate, SubobjectOuter, TemplateName);
							}
							else
							{
								// Fallback if our replacement template was lost
								NewInstance = NewObject<UObject>(SubobjectOuter, NewClass, TemplateName, SubobjectFlags);
								UEngine::CopyPropertiesForUnrelatedObjects(Subobject, NewInstance);
							}

							if (bWasRegistered)
							{
								CastChecked<UActorComponent>(NewInstance)->RegisterComponent();
							}

							ReferenceReplacementMap.Add(Subobject, NewInstance);

							FArchiveReplaceOrClearExternalReferences<UObject> ReplaceAr(SubobjectOuter, ReferenceReplacementMap, SubobjectOuter->GetOutermost());
						}

						GEngine->NotifyToolsOfObjectReplacement(ReferenceReplacementMap);
					}

					void SwapObjects(UObject* Object, const bool bApply)
					{
						const TCHAR* ReplaceName = (bApply ? TEXT("REPLACEMENT") : TEXT("REPLACED"));
						const UClass* CurrentClass = (bApply ? ApplyClass : RevertClass);
						const UClass* NewClass = (bApply ? RevertClass : ApplyClass);

						UBlueprint* BP = CastChecked<UBlueprint>(Object);
						UObject* Template = static_cast<UObject*>(FindObjectWithOuter(BP->GeneratedClass->GetDefaultObject(), nullptr, TemplateName));
						ensure(Template && Template->GetClass() == CurrentClass);

						TArray<UObject*> ArchetypeInstances;
						Template->GetArchetypeInstances(ArchetypeInstances);
						ArchetypeInstances.Add(Template);

						SwapObjects(ArchetypeInstances, ReplaceName, NewClass);

						FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BP);
					}

					virtual void Apply(UObject* Object) override
					{
						SwapObjects(Object, true);
					}

					virtual void Revert(UObject* Object) override
					{
						SwapObjects(Object, false);
					}

					virtual void AddReferencedObjects(FReferenceCollector& Collector) override
					{
						Collector.AddReferencedObject(ApplyClass);
						Collector.AddReferencedObject(RevertClass);
					}

					virtual FString ToString() const override
					{
						return FString();
					}

					FGuid OperationGuid;
					FName TemplateName;
					TObjectPtr<const UClass> ApplyClass;
					TObjectPtr<const UClass> RevertClass;
				};

				// To avoid all of the subobjects we are manipulating getting serialized in to the transaction buffer unnecessarily
				// we record them with a simple custom dummy custom change as we don't actually need them to be transacted since
				// the primary custom change operation on the blueprint will handle destroying and replacing them
				struct FDummySubobjectClassChange : FCommandChange
				{
					virtual void Apply(UObject* Object) override {}
					virtual void Revert(UObject* Object) override {}
					virtual FString ToString() const override { return FString(); }
				};

				const FGuid OperationGuid = FGuid::NewGuid();
				const FString CurrentName = ComponentTemplate->GetName();

				TArray<UObject*> ArchetypeInstances;
				ComponentTemplate->GetArchetypeInstances(ArchetypeInstances);
				ArchetypeInstances.Add(ComponentTemplate);

				TArray<UObject*> ReplacementObjects;
				ReplacementObjects.Reserve(ArchetypeInstances.Num());

				for (UObject* ArchetypeInstance : ArchetypeInstances)
				{
					GUndo->StoreUndo(ArchetypeInstance, MakeUnique<FDummySubobjectClassChange>());

					UObject* SubobjectOuter = ArchetypeInstance->GetOuter();
					SubobjectOuter->Modify();

					{
						const FString ReplacementName = FSubobjectClassChange::MakeReplacementName(CurrentName, SubobjectOuter, TEXT("REPLACEMENT"), OperationGuid);
						const FString ReplacedName = FSubobjectClassChange::MakeReplacementName(CurrentName, SubobjectOuter, TEXT("REPLACED"), OperationGuid);

						// Avoid the new instance going in to the transaction buffer as we are managing that manually through the custom change record
						TGuardValue<ITransaction*> SuppressTransaction(GUndo, nullptr);
						UObject* NewInstance = NewObject<UObject>(GetTransientOuterForRename(const_cast<UClass*>(NewClass)), NewClass, *ReplacementName, ArchetypeInstance->GetFlags());

						UEngine::CopyPropertiesForUnrelatedObjects(ArchetypeInstance, NewInstance);
						ReplacementObjects.Add(NewInstance);

						StaticDuplicateObject(ArchetypeInstance, GetTransientOuterForRename(const_cast<UClass*>(NewClass)), *ReplacedName);
					}
				}

				TUniquePtr<FSubobjectClassChange> SubobjectClassChange = MakeUnique<FSubobjectClassChange>(OperationGuid, ComponentTemplate->GetFName(), ComponentTemplate->GetClass(), NewClass);
				SubobjectClassChange->SwapObjects(ArchetypeInstances, TEXT("REPLACEMENT"), NewClass, ReplacementObjects);

				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BlueprintObj);

				GUndo->StoreUndo(BlueprintObj, MoveTemp(SubobjectClassChange));

				return true;
			}
		}
	}

	return false;
}

bool USubobjectDataSubsystem::ReparentSubobject(const FReparentSubobjectParams& Params, const FSubobjectDataHandle& ToReparentHandle)
{
	TArray<FSubobjectDataHandle> Handles;
	Handles.Add(ToReparentHandle);
	
	return ReparentSubobjects(Params, Handles);
}

bool USubobjectDataSubsystem::MakeNewSceneRoot(const FSubobjectDataHandle& Context, const FSubobjectDataHandle& DroppedNewSceneRootHandle, UBlueprint* Blueprint)
{
	if(!ensure(Context.IsValid()) || !ensure(DroppedNewSceneRootHandle.IsValid()))
	{
		UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to make new scene root: Invalid context or scene root handle!");
		return false;
	}

	FSubobjectData* DroppedNewSceneRootData = DroppedNewSceneRootHandle.GetData();
	
	FSubobjectDataHandle StartingRootHandle = FindSceneRootForSubobject(Context);
	FSubobjectData* StartingRootData = StartingRootHandle.GetData();
	// Remember whether or not we're replacing the default scene root
	bool bWasDefaultSceneRoot = StartingRootData && StartingRootData->IsDefaultSceneRoot();
	
	FSubobjectDataHandle OldSceneRoot = FSubobjectDataHandle::InvalidHandle;
	
	if(Blueprint)
	{
		check(Blueprint && Blueprint->SimpleConstructionScript);

		// Clone the component if it's being dropped into a different subobject tree
		if(DroppedNewSceneRootData->GetBlueprint() != Blueprint)
		{
			UActorComponent* ComponentTemplate = DroppedNewSceneRootData->GetMutableComponentTemplate();
			check(ComponentTemplate);
			FAddNewSubobjectParams Params;
			Params.NewClass = ComponentTemplate->GetClass();
			Params.BlueprintContext = Blueprint;
			Params.AssetOverride = nullptr;
			Params.ParentHandle = Context;
			FText FailReason;
			
			// Note: This will mark the Blueprint as structurally modified
			FSubobjectDataHandle ClonedHandle = AddNewSubobject(Params, FailReason);
			check(ClonedHandle.IsValid());
			UActorComponent* ClonedComponent = ClonedHandle.GetData()->GetMutableComponentTemplate();
			check(ClonedComponent);

			// Serialize object properties using write/read operations.
			UEngine::CopyPropertiesForUnrelatedObjects(ComponentTemplate, ClonedComponent);

			DroppedNewSceneRootData = ClonedHandle.GetData();
			check(DroppedNewSceneRootData->IsValid());
		}

		if(DroppedNewSceneRootData->GetParentHandle().IsValid() && DroppedNewSceneRootData->GetBlueprint() == Blueprint)
		{
			// If the associated component template is a scene component, reset its transform since it will now become the root
			if(USceneComponent* SceneComponentTemplate = Cast<USceneComponent>(DroppedNewSceneRootData->GetMutableComponentTemplate()))
			{
				// Save current state
				SceneComponentTemplate->Modify();

				// Reset the attach socket name
				SceneComponentTemplate->SetupAttachment(SceneComponentTemplate->GetAttachParent(), NAME_None);
				
				if(USCS_Node* SCS_Node = DroppedNewSceneRootData->GetSCSNode())
				{
					SCS_Node->Modify();
					SCS_Node->AttachToName = NAME_None;
				}

				// Cache the current relative location and rotation values (for propagation)
				const FVector OldRelativeLocation = SceneComponentTemplate->GetRelativeLocation();
				const FRotator OldRelativeRotation = SceneComponentTemplate->GetRelativeRotation();

				// Reset the relative transform (location and rotation only; scale is preserved)
				SceneComponentTemplate->SetRelativeLocation(FVector::ZeroVector);
				SceneComponentTemplate->SetRelativeRotation(FRotator::ZeroRotator);

				// Propagate the root change & detachment to any instances of the template (done within the context of the current transaction)
				TArray<UObject*> ArchetypeInstances;
				SceneComponentTemplate->GetArchetypeInstances(ArchetypeInstances);
				FDetachmentTransformRules DetachmentTransformRules(EDetachmentRule::KeepWorld, EDetachmentRule::KeepWorld, EDetachmentRule::KeepRelative, true);
				for (int32 InstanceIndex = 0; InstanceIndex < ArchetypeInstances.Num(); ++InstanceIndex)
				{
					USceneComponent* SceneComponentInstance = Cast<USceneComponent>(ArchetypeInstances[InstanceIndex]);
					if (SceneComponentInstance != nullptr)
					{
						// Detach from root (keeping world transform, except for scale)
						SceneComponentInstance->DetachFromComponent(DetachmentTransformRules);

						// Propagate the default relative location & rotation reset from the template to the instance
						FComponentEditorUtils::ApplyDefaultValueChange(SceneComponentInstance, SceneComponentInstance->GetRelativeLocation_DirectMutable(), OldRelativeLocation, SceneComponentTemplate->GetRelativeLocation());
						FComponentEditorUtils::ApplyDefaultValueChange(SceneComponentInstance, SceneComponentInstance->GetRelativeRotation_DirectMutable(), OldRelativeRotation, SceneComponentTemplate->GetRelativeRotation());

						// Must also reset the root component here, so that RerunConstructionScripts() will cache the correct root component instance data
						AActor* Owner = SceneComponentInstance->GetOwner();
						if (Owner)
						{
							Owner->Modify();
							Owner->SetRootComponent(SceneComponentInstance);
						}
					}
				}
			}
			
			// Remove the dropped node from its existing parent
			DetachSubobject(DroppedNewSceneRootData->GetParentHandle(), DroppedNewSceneRootData->GetHandle());
		}

		check(StartingRootData && (bWasDefaultSceneRoot || StartingRootData->CanReparent()));

		// Remove the current scene root node from the SCS context
		Blueprint->SimpleConstructionScript->RemoveNode(StartingRootData->GetSCSNode(), /*bValidateSceneRootNodes=*/false);

		// Save old root node
		OldSceneRoot = StartingRootData->GetHandle();

		// Add dropped node to the SCS context
		Blueprint->SimpleConstructionScript->AddNode(DroppedNewSceneRootData->GetSCSNode());
		
		// Remove or re-parent the old root
		if (OldSceneRoot.IsValid())
		{
			check(DroppedNewSceneRootData->CanReparent());

			// Set old root as child of new root
			AttachSubobject(DroppedNewSceneRootData->GetHandle(), OldSceneRoot);
			FSubobjectData* OldData = OldSceneRoot.GetData();

			// If the old SCS node was the default scene root created by every blueprint, then we can delete it
			// Otherwise we will keep it and simply reparent it to the new scene root
			if (OldData->GetSCSNode() == Blueprint->SimpleConstructionScript->GetDefaultSceneRootNode())
			{
				TArray<FSubobjectDataHandle> ToDelete { OldSceneRoot };
				FSubobjectDataHandle NewSelection = FSubobjectDataHandle::InvalidHandle;
				DeleteSubobjects(Context, ToDelete, NewSelection, Blueprint, /* bForce */ true);
			}
		}
	}
	else
	{
		// Remove the dropped node from its existing parent
		if(DroppedNewSceneRootData->HasParent())
		{
			DetachSubobject(DroppedNewSceneRootData->GetParentHandle(), DroppedNewSceneRootData->GetHandle());
		}
		
		// Save old root node
		OldSceneRoot = StartingRootHandle;

		if(OldSceneRoot.IsValid())
		{
			// If the thing we replaced was the default scene root, then just delete it
			if(bWasDefaultSceneRoot)
			{
				// Destroy the old root component instance
				UActorComponent* OldRootInstance = OldSceneRoot.GetData()->GetMutableComponentTemplate();
				OldRootInstance->Modify();
				OldRootInstance->DestroyComponent(!bWasDefaultSceneRoot);

				// Set the new root component
				AActor* ActorContext = Context.GetData()->GetMutableActorContext();
				ActorContext->SetRootComponent(CastChecked<USceneComponent>(DroppedNewSceneRootData->GetMutableComponentTemplate()));
			}
			// Otherwise, reparent it to the new scene root
			else
			{
				FReparentSubobjectParams Params;
				Params.BlueprintContext = nullptr;
				Params.ActorPreviewContext = nullptr;
				Params.NewParentHandle = DroppedNewSceneRootHandle;
				
				ReparentSubobject(Params, OldSceneRoot);
			}
		}
	}

	return true;
}

bool USubobjectDataSubsystem::ReparentSubobjects(const FReparentSubobjectParams& Params, const TArray<FSubobjectDataHandle>& HandlesToMove)
{
	if(!Params.NewParentHandle.IsValid())
	{
		UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to reparent: Invalid parent handle when reparenting!");
		return false;
	}
	
	FSubobjectData* NewParentData = Params.NewParentHandle.GetData();

	if (!ensureMsgf(NewParentData, TEXT("There was no valid parent data given! Exiting...")))
	{
		return false;
	}
	
	if (Params.BlueprintContext)
	{
		if (!Params.ActorPreviewContext)
		{
			UE_LOGF(LogSubobjectSubsystem, Warning, "Failed to reparent: In a blueprint context there must be an actor preview!");
			return false;
		}

		for (const FSubobjectDataHandle& HandleToMove : HandlesToMove)
		{
			FSubobjectData* DroppedData = HandleToMove.GetSharedDataPtr().Get();
			if(DroppedData->GetBlueprint() != Params.BlueprintContext)
			{
				UActorComponent* ComponentTemplate = DroppedData->GetMutableComponentTemplate();
				check(ComponentTemplate);
	
				FAddNewSubobjectParams AddNewParms;
				AddNewParms.BlueprintContext = Params.BlueprintContext;
				AddNewParms.NewClass = ComponentTemplate->GetClass();
				AddNewParms.ParentHandle = NewParentData->GetHandle();
				
				// Note: This will mark the Blueprint as structurally modified
				FText FailReason;
				FSubobjectDataHandle ClonedSubobject = AddNewSubobject(AddNewParms, FailReason);
				check(ClonedSubobject.IsValid());
				
				FSubobjectData* ClonedData = ClonedSubobject.GetSharedDataPtr().Get();
				UActorComponent* ClonedComponent = ClonedData->GetMutableComponentTemplate();
	
				UEngine::CopyPropertiesForUnrelatedObjects(ComponentTemplate, ClonedComponent);		
			}
			else
			{
				USceneComponent* SceneComponentTemplate = Cast<USceneComponent>(DroppedData->GetMutableComponentTemplate());
				// Cache current default values for propagation
				FVector OldRelativeLocation, OldRelativeScale3D;
				FRotator OldRelativeRotation;
				if(SceneComponentTemplate)
				{
					OldRelativeLocation = SceneComponentTemplate->GetRelativeLocation();
					OldRelativeRotation = SceneComponentTemplate->GetRelativeRotation();
					OldRelativeScale3D = SceneComponentTemplate->GetRelativeScale3D();
				}

				FSubobjectDataHandle OldParentHandle = DroppedData->GetParentHandle();
				FSubobjectData* OldParentData = OldParentHandle.IsValid() ? OldParentHandle.GetSharedDataPtr().Get() : nullptr;

				if(OldParentData)
				{
					DetachSubobject(OldParentHandle, DroppedData->GetHandle());
					// If the associated component template is a scene component, maintain its preview world position
					if(SceneComponentTemplate)
					{
						// Save current state
						SceneComponentTemplate->Modify();

						// Reset the attach socket name
						SceneComponentTemplate->SetupAttachment(SceneComponentTemplate->GetAttachParent(), NAME_None);
						USCS_Node* SCS_Node = DroppedData->GetSCSNode();
						if(SCS_Node)
						{
							SCS_Node->Modify();
							SCS_Node->AttachToName = NAME_None;
						}

						// Attempt to locate a matching registered instance of the component template in the Actor context that's being edited
						const USceneComponent* InstancedSceneComponent = Cast<USceneComponent>(DroppedData->FindMutableComponentInstanceInActor(Params.ActorPreviewContext));
						if(InstancedSceneComponent && InstancedSceneComponent->IsRegistered())
						{
							// If we find a match, save off the world position
							const FTransform& ComponentToWorld = InstancedSceneComponent->GetComponentToWorld();
							SceneComponentTemplate->SetRelativeTransform_Direct(ComponentToWorld);
						}
					}
				}

				// Attach the dropped node to the given node
				AttachSubobject(NewParentData->GetHandle(), DroppedData->GetHandle());

				// Attempt to locate a matching instance of the parent component template in the Actor context that's being edited
				USceneComponent* ParentSceneComponent = Cast<USceneComponent>(NewParentData->FindMutableComponentInstanceInActor(Params.ActorPreviewContext));
				if(SceneComponentTemplate && ParentSceneComponent && ParentSceneComponent->IsRegistered())
				{
					ConformTransformRelativeToParent(SceneComponentTemplate, ParentSceneComponent);
				}

				// Propagate any default value changes out to all instances of the template. If we didn't do this, then instances could incorrectly override the new default value with the old default value when construction scripts are re-run.
				if(SceneComponentTemplate)
				{
					TArray<UObject*> InstancedSceneComponents;
					SceneComponentTemplate->GetArchetypeInstances(InstancedSceneComponents);
					for(int32 InstanceIndex = 0; InstanceIndex < InstancedSceneComponents.Num(); ++InstanceIndex)
					{
						USceneComponent* InstancedSceneComponent = Cast<USceneComponent>(InstancedSceneComponents[InstanceIndex]);
						if(InstancedSceneComponent != nullptr)
						{
							FComponentEditorUtils::ApplyDefaultValueChange(InstancedSceneComponent, InstancedSceneComponent->GetRelativeLocation_DirectMutable(), OldRelativeLocation, SceneComponentTemplate->GetRelativeLocation());
							FComponentEditorUtils::ApplyDefaultValueChange(InstancedSceneComponent, InstancedSceneComponent->GetRelativeRotation_DirectMutable(), OldRelativeRotation, SceneComponentTemplate->GetRelativeRotation());
							FComponentEditorUtils::ApplyDefaultValueChange(InstancedSceneComponent, InstancedSceneComponent->GetRelativeScale3D_DirectMutable(),  OldRelativeScale3D,  SceneComponentTemplate->GetRelativeScale3D());
						}
					}
				}
			}
		}

		FBlueprintEditorUtils::PostEditChangeBlueprintActors(Params.BlueprintContext, true);
		return true;
	}
	else
	{
		for (const FSubobjectDataHandle& HandleToMove : HandlesToMove)
		{
			// Remove this handle from it's parent if it has one
			FSubobjectData* DataToMove = HandleToMove.GetData();

			if (DataToMove->HasParent())
			{
				DetachSubobject(DataToMove->GetParentHandle(), HandleToMove);
			}

			AttachSubobject(Params.NewParentHandle, HandleToMove);
		}

		// Rerun construction scripts on the base actor 
		if (AActor* ActorInstance = NewParentData->GetMutableActorContext())
		{
			ActorInstance->RerunConstructionScripts();
		}

		BroadcastInstanceChanges();
	}
	
	return true;
}

bool USubobjectDataSubsystem::DetachSubobject(const FSubobjectDataHandle& OwnerHandle, const FSubobjectDataHandle& ChildToRemove)
{
	FSubobjectData* OwnerData = OwnerHandle.GetData();
	FSubobjectData* ChildToRemoveData = ChildToRemove.GetData();
	if(!OwnerData || !ChildToRemoveData)
	{
		return false;
	}

	// Remove this subobject handle from the parent
	OwnerData->RemoveChildHandleOnly(ChildToRemoveData->GetHandle());
	ChildToRemoveData->ClearParentHandle();
	// if its an instance component, call detach from component
	if(ChildToRemoveData->IsInstancedComponent())
	{
		USceneComponent* ChildInstance = Cast<USceneComponent>(ChildToRemoveData->GetMutableComponentTemplate());
		if (ensure(ChildInstance))
		{
			// Handle detachment at the instance level
			ChildInstance->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
		}
		return true;
	}

	// Bypass removal logic if we're part of a child actor subtree
	if (ChildToRemoveData->IsChildActorSubtreeObject())
	{
		return true;
	}
	
	if(USCS_Node* SCS_ChildNode = ChildToRemoveData->GetSCSNode())
	{
		USimpleConstructionScript* SCS = SCS_ChildNode->GetSCS();
		
		if (SCS != nullptr)
		{
			SCS->RemoveNode(SCS_ChildNode);
		}
	}
	
	return true;
}

bool USubobjectDataSubsystem::AttachSubobject(const FSubobjectDataHandle& OwnerHandle, const FSubobjectDataHandle& ChildToAddHandle)
{
	FSubobjectData* OwnerData = OwnerHandle.GetData();
	FSubobjectData* ChildToAddData = ChildToAddHandle.GetData();

	if(!OwnerData || !ChildToAddData)
	{
		return false;
	}

	// If the given child has a parent already, remove it from that first
	if(ChildToAddData->HasParent())
	{
		DetachSubobject(ChildToAddData->GetParentHandle(), ChildToAddHandle);
	}

	check(!ChildToAddData->HasParent());
	
	// Reparent it to the new data
	OwnerData->AddChildHandleOnly(ChildToAddHandle);
	ChildToAddData->SetParentHandle(OwnerHandle);
	
	if(ChildToAddData->IsComponent())
	{
		const UActorComponent* ComponentTemplate = OwnerData->GetObject<UActorComponent>();
		USceneComponent* ParentInstance = nullptr;
		bool bIsInstancedComponent = OwnerData->IsInstancedComponent();

		if (ComponentTemplate)
		{
			// Find the component instance on the current actor if we can
			const FSubobjectDataHandle& ActorHandle = GetActorRootHandle(OwnerData->GetHandle());
			const FSubobjectData* ActorData = ActorHandle.GetData();
			ParentInstance = ActorData ? Cast<USceneComponent>(OwnerData->FindMutableComponentInstanceInActor(ActorData->GetObject<AActor>())) : nullptr;
			bIsInstancedComponent |= ParentInstance != nullptr;
		}

		// Add a child node to the SCS tree node if not already present
		if(USCS_Node* SCS_ChildNode = ChildToAddData->GetSCSNode())
		{
			USCS_Node* SCS_Node = OwnerData->GetSCSNode();
			// Get the SCS instance that owns the child node
			if(USimpleConstructionScript* SCS = SCS_ChildNode->GetSCS())
			{
				if(SCS_Node)
				{
					// If the parent and child are both owned by the same SCS instance
					if (SCS_Node->GetSCS() == SCS)
					{
						// Add the child into the parent's list of children
						if (!SCS_Node->GetChildNodes().Contains(SCS_ChildNode))
						{
							SCS_Node->AddChildNode(SCS_ChildNode);
						}
					}
					else
					{
						// Adds the child to the SCS root set if not already present
						SCS->AddNode(SCS_ChildNode);

						// Set parameters to parent this node to the "inherited" SCS node
						SCS_ChildNode->SetParent(SCS_Node);
					}
				}
				else if(ComponentTemplate)
				{
					// Adds the child to the SCS root set if not already present
					SCS->AddNode(SCS_ChildNode);
					
					// Set parameters to parent this node to the native component template
					SCS_ChildNode->SetParent(Cast<const USceneComponent>(ComponentTemplate));
				}
				else
				{
					// Adds the child to the SCS root set if not already present
					SCS->AddNode(SCS_ChildNode);
				}
			}
		}
		else if(bIsInstancedComponent)
		{
			USceneComponent* ChildInstance = Cast<USceneComponent>(ChildToAddData->GetMutableComponentTemplate());

			if (ensure(ChildInstance && ParentInstance))
			{
				// Handle attachment at the instance level
				if (ChildInstance->GetAttachParent() != ParentInstance)
				{
					AActor* Owner = ParentInstance->GetOwner();
					if (Owner->GetRootComponent() == ChildInstance)
					{
						Owner->SetRootComponent(ParentInstance);
					}
					ChildInstance->AttachToComponent(ParentInstance, FAttachmentTransformRules::KeepWorldTransform);
				}	
			}
		}
	}
	
	return true;
}

bool USubobjectDataSubsystem::IsValidRename(const FSubobjectDataHandle& Handle, const FText& InNewText, FText& OutErrorMessage) const
{
	const FSubobjectData* Data = Handle.GetData();
	if(!Data)
	{
		return false;		
	}
	
	const UBlueprint* Blueprint = Data->GetBlueprint();
	const FString& NewTextStr = InNewText.ToString();

	if (!NewTextStr.IsEmpty())
	{
		if (Data->GetVariableName().ToString() == NewTextStr)
		{
			return true;
		}

		if (const UActorComponent* ComponentInstance = Data->GetComponentTemplate())
		{
			AActor* ExistingNameSearchScope = ComponentInstance->GetOwner();
			if ((ExistingNameSearchScope == nullptr) && (Blueprint != nullptr))
			{
				ExistingNameSearchScope = Cast<AActor>(Blueprint->GeneratedClass->GetDefaultObject());
			}

			if (!FComponentEditorUtils::IsValidVariableNameString(ComponentInstance, NewTextStr))
			{
				OutErrorMessage = LOCTEXT("RenameFailed_EngineReservedName", "This name is reserved for engine use.");
				return false;
			}
			else if (NewTextStr.Len() >= NAME_SIZE)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("CharCount"), NAME_SIZE - 1);
				OutErrorMessage = FText::Format(LOCTEXT("ComponentRenameFailed_TooLong", "Component name must be less than {CharCount} characters long."), Arguments);
				return false;
			}
			else if (!FComponentEditorUtils::IsComponentNameAvailable(NewTextStr, ExistingNameSearchScope, ComponentInstance) 
                    || !FComponentEditorUtils::IsComponentNameAvailable(NewTextStr, ComponentInstance->GetOuter(), ComponentInstance ))
			{
				OutErrorMessage = LOCTEXT("RenameFailed_ExistingName", "Another component already has the same name.");
				return false;
			}
		}
		else if(const AActor* ActorInstance = Data->GetObject<AActor>())
		{
			// #TODO_BH Validation of actor instance
		}
		else
		{
			OutErrorMessage = LOCTEXT("RenameFailed_InvalidComponentInstance", "This node is referencing an invalid component instance and cannot be renamed. Perhaps it was destroyed?");
			return false;
		}
	}
	
	TSharedPtr<INameValidatorInterface> NameValidator;
	if (Blueprint != nullptr)
	{
		NameValidator = MakeShareable(new FKismetNameValidator(Blueprint, Data->GetVariableName()));
	}
	else if(const UActorComponent* CompTemplate = Data->GetComponentTemplate())
	{
		NameValidator = MakeShareable(new FStringSetNameValidator(CompTemplate->GetName()));
	}

	if(NameValidator)
	{
		EValidatorResult ValidatorResult = NameValidator->IsValid(NewTextStr);
		if (ValidatorResult == EValidatorResult::AlreadyInUse)
		{
			OutErrorMessage = FText::Format(LOCTEXT("RenameFailed_InUse", "{0} is in use by another variable or function!"), InNewText);
		}
		else if (ValidatorResult == EValidatorResult::EmptyName)
		{
			OutErrorMessage = LOCTEXT("RenameFailed_LeftBlank", "Names cannot be left blank!");
		}
		else if (ValidatorResult == EValidatorResult::TooLong)
		{
			OutErrorMessage = LOCTEXT("RenameFailed_NameTooLong", "Names must have fewer than 100 characters!");
		}
	}

	if (OutErrorMessage.IsEmpty())
	{
		return true;
	}

	return false;
}

bool USubobjectDataSubsystem::CanCopySubobjects(const TArray<FSubobjectDataHandle>& Handles) const
{
	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_CanCopySubobjects(Handles))
		{
			return Entry.Adapter->CanCopySubobjects(Handles);
		}
	}
		
	TArray<UActorComponent*> ComponentsToCopy;
	
	for (const FSubobjectDataHandle& Handle : Handles)
    {
    	if(FSubobjectData* Data = Handle.GetData())
    	{
    		if(!Data->CanCopy())
    		{
    			return false;
    		}

			// Get the component template associated with the selected node
			UActorComponent* ComponentTemplate = nullptr;
			const FSubobjectDataHandle& ActorRootHandle = GetActorRootHandle(Handle);
			if (const FSubobjectData* ActorData = ActorRootHandle.GetData())
			{
				ComponentTemplate = Data->FindMutableComponentInstanceInActor(ActorData->GetObject<AActor>());
			}
			
			if (!ComponentTemplate)
			{
				ComponentTemplate = Data->GetMutableComponentTemplate();
			}
			
			if (ComponentTemplate)
			{
				ComponentsToCopy.Add(ComponentTemplate);
			}
		}
	}

	// Verify that the components can be copied
	return FComponentEditorUtils::CanCopyComponents(ComponentsToCopy);
}

void USubobjectDataSubsystem::CopySubobjects(const TArray<FSubobjectDataHandle>& Handles, UBlueprint* BpContext)
{
	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_CopySubobjects(Handles, BpContext))
		{
			Entry.Adapter->CopySubobjects(Handles, BpContext);
			return;
		}
	}
	
	if(!CanCopySubobjects(Handles))
	{
		return;
	}

	TArray<UActorComponent*> ComponentsToCopy;

	for (const FSubobjectDataHandle& Handle : Handles)
	{
		if(FSubobjectData* Data = Handle.GetData())
		{
			ensureMsgf(Data->CanCopy(), TEXT("A non-copiable subobject has been allowed to copy!"));

			// Get the component template associated with the selected node
			UActorComponent* ComponentTemplate = nullptr;
			const FSubobjectDataHandle& ActorRootHandle = GetActorRootHandle(Handle);
			if (const FSubobjectData* ActorData = ActorRootHandle.GetData())
			{
				ComponentTemplate = Data->FindMutableComponentInstanceInActor(ActorData->GetObject<AActor>());
			}
			
			if (!ComponentTemplate)
			{
				ComponentTemplate = Data->GetMutableComponentTemplate();
			}
    		
			// Get the component template associated with the selected node
			if (ComponentTemplate)
			{
				ComponentsToCopy.Add(ComponentTemplate);
				if(BpContext && ComponentTemplate->CreationMethod != EComponentCreationMethod::UserConstructionScript)
				{
					// CopyComponents uses component attachment to maintain hierarchy, but the SCS templates are not
					// setup with a relationship to each other. Briefly setup the attachment between the templates being
					// copied so that the hierarchy is retained upon pasting
					if (USceneComponent* SceneTemplate = Cast<USceneComponent>(ComponentTemplate))
					{
						if (FSubobjectData* SelectedParentNodePtr = Data->GetParentHandle().GetData())
						{
							if (USceneComponent* ParentSceneTemplate = Cast<USceneComponent>(SelectedParentNodePtr->GetMutableComponentTemplate()))
							{
								SceneTemplate->SetupAttachment(ParentSceneTemplate);
							}
						}
					}
				}
			}
		}
	}
	
	// Copy the components to the clipboard
	FComponentEditorUtils::CopyComponents(ComponentsToCopy);
	
	if(BpContext)
	{
		for (UActorComponent* ComponentTemplate : ComponentsToCopy)
		{
			if (ComponentTemplate->CreationMethod != EComponentCreationMethod::UserConstructionScript)
			{
				if (USceneComponent* SceneTemplate = Cast<USceneComponent>(ComponentTemplate))
				{
					// clear back out any temporary attachments we set up for the copy
					SceneTemplate->SetupAttachment(nullptr);
				}
			}
		}
	}
}

bool USubobjectDataSubsystem::CanPasteSubobjects(const FSubobjectDataHandle& RootHandle, UBlueprint* BPContext) const
{
	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_CanPasteSubobjects(RootHandle, BPContext))
		{
			return Entry.Adapter->CanPasteSubobjects(RootHandle, BPContext);
		}
	}
	
	const FSubobjectDataHandle& SceneRootHandle = FindSceneRootForSubobject(RootHandle);
	const FSubobjectData* RootData = SceneRootHandle.GetData();
	const USceneComponent* SceneComponent = Cast<USceneComponent>(RootData->GetComponentTemplate());
	if(const AActor* RootActor = RootData->GetObject<AActor>())
	{
		SceneComponent = RootActor->GetRootComponent();
	}
	
	return
		(BPContext && RootData->IsActor())  ||
		(SceneComponent && FComponentEditorUtils::CanPasteComponents(SceneComponent, RootData->IsDefaultSceneRoot(), true));
}

void USubobjectDataSubsystem::PasteSubobjects(const FSubobjectDataHandle& PasteToContext, const TArray<FSubobjectDataHandle>& NewParentHandles, UBlueprint* Blueprint, TArray<FSubobjectDataHandle>& OutPastedHandles)
{
	if(!PasteToContext.IsValid() || NewParentHandles.IsEmpty())
	{
		return;
	}

	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_PasteSubobjects(PasteToContext, NewParentHandles, Blueprint))
		{
			Entry.Adapter->PasteSubobjects(PasteToContext, NewParentHandles, Blueprint, OutPastedHandles);
			return;
		}
	}
	
	FSubobjectData* PasteToContextData = PasteToContext.GetSharedDataPtr().Get();	
	
	const FScopedTransaction Transaction(LOCTEXT("PasteComponents", "Paste Component(s)"));
	if(Blueprint)
	{
		// Get the components to paste from the clipboard
		TMap<FName, FName> ParentMap;
		TMap<FName, UActorComponent*> NewObjectMap;
		FComponentEditorUtils::GetComponentsFromClipboard(ParentMap, NewObjectMap, true);

		check(Blueprint->SimpleConstructionScript != nullptr);
		
		Blueprint->Modify();
		SaveSCSCurrentState(Blueprint->SimpleConstructionScript);
		
		// Create a new tree node for each new (pasted) component
        FSubobjectDataHandle FirstNode;
        TMap<FName, FSubobjectDataHandle> NewNodeMap;

		for (const TPair<FName, UActorComponent*>& NewObjectPair : NewObjectMap)
		{
			// Get the component object instance
			UActorComponent* NewActorComponent = NewObjectPair.Value;
			check(NewActorComponent);
			
			// make sure creation method is set to SimpleConstructionScript
			if (UE::SubobjectDataSubsystem::bForcePastedComponentsToSCS)
			{
				if (NewActorComponent->CreationMethod == EComponentCreationMethod::Instance)
				{
					NewActorComponent->CreationMethod = EComponentCreationMethod::SimpleConstructionScript;
				}
			}
			
			// Create a new SCS node to contain the new component and add it to the tree
			USCS_Node* NewSCSNode = Blueprint->SimpleConstructionScript->CreateNodeAndRenameComponent(NewActorComponent);
			if(NewSCSNode)
			{
				NewSCSNode->Modify();
				NewActorComponent = ToRawPtr(NewSCSNode->ComponentTemplate);
			}
			else
			{
				NewActorComponent = nullptr;
			}

			// Find a suitable parent handle to be used as the default, which would be the root actor
			FSubobjectDataHandle TargetParentHandle = FindParentForNewSubobject(NewActorComponent, PasteToContext);

			// Ensure that for scene components they are attached to the scene root. Non-Scene components (i.e. a blackboard)
			// should stay attached to the root actor, not the scene root.
			if (NewActorComponent && NewActorComponent->IsA<USceneComponent>())
			{
				// A scene component should be attaching to the scene root by default.
				TargetParentHandle = FindSceneRootForSubobject(TargetParentHandle);

				// If there were any handles that should be considered instead, see if they are valid.
				for (FSubobjectDataHandle SelectedNode : NewParentHandles)
				{
					const FSubobjectData* SelectedData = SelectedNode.GetSharedDataPtr().Get();

					// Only scene components can be attached to when pasting
					if (SelectedData && SelectedData->IsSceneComponent() && SelectedData->HasParent())
					{
						const FSubobjectData* ParentData = SelectedData->GetParentHandle().GetData();
						// The parent data can only be used if it's a scene component (not an actor or non-scene component)
						if (ParentData && ParentData->IsSceneComponent())
						{
							TargetParentHandle = SelectedData->GetParentHandle();
							break;
						}
					}
				}
			}

			TSharedPtr<FSubobjectData> TargetData = TargetParentHandle.GetSharedDataPtr();
			
			// Create a new subobject data set with this component. Use the SCS node here and the subobject data
			// will correctly associate the component template
			FSubobjectDataHandle NewDataHandle = FactoryCreateSubobjectDataWithParent(
				NewSCSNode,
				TargetParentHandle,
				TargetData ? TargetData->IsInheritedSCSNode() : false
			);

			AttachSubobject(TargetParentHandle, NewDataHandle);

			OutPastedHandles.Add(NewDataHandle);
			// Map the new subobject's data handle to it's instance name
			NewNodeMap.Add(NewObjectPair.Key, NewDataHandle);
		}

		// Restore the node hierarchy from the original copy
		for (const TPair<FName, FSubobjectDataHandle>& NewNodePair : NewNodeMap)
		{
			// If an entry exists in the set of known parent nodes for the current node
			if (ParentMap.Contains(NewNodePair.Key))
			{
				// Get the parent node name
				FName ParentName = ParentMap[NewNodePair.Key];
				if (NewNodeMap.Contains(ParentName))
				{
					// Reattach the current node to the parent node (this will also handle detachment from the scene root node)
					const FSubobjectDataHandle& DesiredParentHandle = NewNodeMap[ParentName];
					AttachSubobject(DesiredParentHandle, NewNodePair.Value);
				}
			}
		}

		// Modify the Blueprint generated class structure (this will also call UpdateTree() as a result)
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
	}
	else if(AActor* ActorContext = PasteToContextData->GetMutableObject<AActor>())
	{
		// Determine where in the hierarchy to paste (default to the root)
		USceneComponent* TargetComponent = ActorContext->GetRootComponent();
		for (FSubobjectDataHandle SelectedNode : NewParentHandles)
		{
			FSubobjectData* SelectedData = SelectedNode.GetSharedDataPtr().Get();
			check(SelectedData);
		
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(SelectedData->GetMutableComponentTemplate()))
			{
				TargetComponent = SceneComponent;
				break;
			}
		}
		
		// Paste the components. This uses the current data from the clipboard
		TArray<UActorComponent*> PastedComponents;
		FComponentEditorUtils::PasteComponents(PastedComponents, ActorContext, TargetComponent);

		// Create handles for each pasted subobject, attaching it to it's parent as necessary
		for(UActorComponent* PastedComponent : PastedComponents)
		{
			FSubobjectDataHandle ParentHandle = FindHandleForObject(PasteToContext, PastedComponent->GetOuter());
			FSubobjectDataHandle PastedHandle = FactoryCreateSubobjectDataWithParent(PastedComponent, ParentHandle);
			if(PastedHandle.IsValid())
			{
				OutPastedHandles.AddUnique(PastedHandle);
			}
		}
	}
}

bool USubobjectDataSubsystem::CanDuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup)
{
	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_CanDuplicateSubobjects(Context, SubobjectsToDup))
		{
			return Entry.Adapter->CanDuplicateSubobjects(Context, SubobjectsToDup);
		}
	}

	// Default behaviour inherited from SSubobjectEditor
	return CanCopySubobjects(SubobjectsToDup);
}

void USubobjectDataSubsystem::DuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup, UBlueprint* BpContext, TArray<FSubobjectDataHandle>& OutNewSubobjects)
{
	if(!Context.IsValid() || SubobjectsToDup.IsEmpty())
	{
		return;
	}

	for (const auto& Entry : SubsystemAdapters)
	{
		if (Entry.Adapter->CanIntercept_DuplicateSubobjects(Context, SubobjectsToDup, BpContext))
		{
			return Entry.Adapter->DuplicateSubobjects(Context, SubobjectsToDup, BpContext, OutNewSubobjects);
		}
	}

	FAddNewSubobjectParams NewSubobjectParams;
	NewSubobjectParams.BlueprintContext = BpContext;
	NewSubobjectParams.ParentHandle = Context;
	NewSubobjectParams.bConformTransformToParent = false;

	// If we have a valid BP context, defer this step until after the AddNewSubobject() call, because we want
	// to first fix up the template hierarchy (below) before we re-run construction scripts on any instances.
	if (BpContext)
	{
		NewSubobjectParams.bSkipMarkBlueprintModified = true;
	}
	
	FText FailedAddReason = FText::GetEmpty();
	
	TMap<FSubobjectData*, FSubobjectData*> DuplicateSceneComponentMap;

	// For each Subobject to dup, add it as a new subobject to the context
	for(const FSubobjectDataHandle& OriginalHandle : SubobjectsToDup)
	{
		if(!OriginalHandle.IsValid())
		{
			UE_LOGF(LogSubobjectSubsystem, Warning, "Could not duplicate one or more subobjects, an invalid SubobjectToDup was given!");
			continue;
		}

		FSubobjectData* OriginalData = OriginalHandle.GetSharedDataPtr().Get();
		NewSubobjectParams.ParentHandle = OriginalData->GetHandle();

		if(UActorComponent* ComponentTemplate = OriginalData->GetMutableComponentTemplate())
		{
			USCS_Node* SCSNode = OriginalData->GetSCSNode();
			check(SCSNode == nullptr || SCSNode->ComponentTemplate == ComponentTemplate);

			NewSubobjectParams.NewClass = ComponentTemplate->GetClass();
			if (BpContext)
			{
				NewSubobjectParams.AssetOverride = SCSNode ? (UObject*)SCSNode : ComponentTemplate;
			}
			else
			{
				FSubobjectData* ContextData = Context.GetData();
				AActor* ActorContext = ContextData->GetMutableActorContext();
				UActorComponent* InstanceComponent = OriginalData->FindMutableComponentInstanceInActor(ActorContext);
				
				NewSubobjectParams.AssetOverride = InstanceComponent;
			}
			
			FSubobjectDataHandle ClonedSubobject = AddNewSubobject(NewSubobjectParams, FailedAddReason);
			if (ClonedSubobject.IsValid())
			{
				OutNewSubobjects.Add(ClonedSubobject);
			}

			FSubobjectData* ClonedData = ClonedSubobject.GetSharedDataPtr().Get();
			
			if(ClonedData && ClonedData->IsSceneComponent())
			{
				DuplicateSceneComponentMap.Add(OriginalData, ClonedData);
			}
		}
	}

	for (const TPair<FSubobjectData*, FSubobjectData*>& DuplicatedPair : DuplicateSceneComponentMap)
	{
		FSubobjectData* OriginalData = DuplicatedPair.Key;
		FSubobjectData* NewData = DuplicatedPair.Value;

		USceneComponent* OriginalComponent = CastChecked<USceneComponent>(OriginalData->GetMutableComponentTemplate());
		USceneComponent* NewSceneComponent = CastChecked<USceneComponent>(NewData->GetMutableComponentTemplate());
		
		if(BpContext)
		{
			// Ensure that any native attachment relationship inherited from the original copy is removed (to prevent a GLEO assertion)
			NewSceneComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
		}

		// If we're duplicating the root then we're already a child of it so need to reparent, but we do need to reset the scale
		// otherwise we'll end up with the square of the root's scale instead of being the same size.
		if(OriginalData->IsDefaultSceneRoot())
		{
			NewSceneComponent->SetRelativeScale3D_Direct(FVector(1.f));
		}
		else
		{
			// If the original node was parented, attempt to add the duplicate as a child of the same parent node if the parent is not
			// part of the duplicate set, otherwise parent to the parent's duplicate
			FSubobjectDataHandle ParentHandle = OriginalData->GetParentHandle();
			if(ParentHandle.IsValid())
			{
				AttachSubobject(ParentHandle, NewData->GetHandle());
			}
		}
	}

	// Now that the hierarchy has been fixed up, we can go ahead and mark the BP as modified (if valid). This in turn will re-run
	// construction scripts on any instances of the Blueprint, whose hierarchies will also now include the new (duplicate) subobject.
	if (BpContext)
	{
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(BpContext);
	}
}

FScopedTransaction* USubobjectDataSubsystem::BeginTransaction(const TArray<FSubobjectDataHandle>& Handles, const FText& Description, UBlueprint* InBlueprint)
{
	FScopedTransaction* OutTransaction = new FScopedTransaction(Description);
	if(InBlueprint)
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(InBlueprint);
	}

	for (const FSubobjectDataHandle& Handle : Handles)
	{
		if(FSubobjectData* Data = Handle.GetData())
		{
			if(USCS_Node* SCS_Node = Data->GetSCSNode())
			{
				USimpleConstructionScript* SCS = SCS_Node->GetSCS();
				UBlueprint* Blueprint = SCS ? SCS->GetBlueprint() : nullptr;
				if (Blueprint == InBlueprint)
				{
					SCS_Node->Modify();
				}
			}

			// Modify template, any instances will be reconstructed as part of PostUndo:
			if (UActorComponent* ComponentTemplate = Data->GetMutableObjectForBlueprint<UActorComponent>(InBlueprint))
			{
				ComponentTemplate->SetFlags(RF_Transactional);
				ComponentTemplate->Modify();
			}
		}
	}
	return OutTransaction;
}

FSubobjectDataHandle USubobjectDataSubsystem::CreateSubobjectData()
{
	TSharedPtr<FSubobjectData> SharedPtr = MakeShared<FSubobjectData>();
	SharedPtr->Handle = SharedPtr;
	return SharedPtr->GetHandle();
}

FSubobjectDataHandle USubobjectDataSubsystem::CreateSubobjectData(UObject* Context, const FSubobjectDataHandle& ParentHandle /* = FSubobjectDataHandle::InvalidHandle */, bool bIsInheritedSCS/* = false */)
{
	TSharedPtr<FSubobjectData> SharedPtr = TSharedPtr<FSubobjectData>(new FSubobjectData());
	SharedPtr->Initialize(Context, ParentHandle, bIsInheritedSCS);
	
	check(SharedPtr.IsValid());
	
	SharedPtr->Handle = SharedPtr;

	return SharedPtr->GetHandle();
}

FSubobjectDataHandle USubobjectDataSubsystem::FactoryCreateSubobjectDataWithParent(UObject* Context, const FSubobjectDataHandle& ParentHandle, bool bIsInheritedSCS /* = false */)
{
	FSubobjectData* ParentData = ParentHandle.GetSharedDataPtr().Get();
	if (!ensureMsgf(ParentData, TEXT("Attempted to use an invalid parent subobject handle!")))
	{
		return FSubobjectDataHandle::InvalidHandle;
	}

	// Does this parent subobject already have this object listed in the children?
	// if yes, then just return that handle
	const FSubobjectDataHandle& ExistingChild = ParentData->FindChildByObject(Context);
	if (ExistingChild.IsValid())
	{
		return ExistingChild;
	}

	// Otherwise, we need to create a new handle
	FSubobjectDataHandle OutHandle = CreateSubobjectData(Context, ParentHandle, bIsInheritedSCS);
	
	// Inform the parent that it has a new child
	const bool bSuccess = ParentData->AddChildHandleOnly(OutHandle);
	ensureMsgf(bSuccess, TEXT("Failed to add a child to parent subobject!"));

	return OutHandle;
}

FSubobjectDataHandle USubobjectDataSubsystem::FactoryCreateInheritedBpSubobject(UObject* Context, const FSubobjectDataHandle& InParentHandle, bool bIsInherited, TArray<FSubobjectDataHandle>& OutArray)
{
	USCS_Node* InSCSNode = Cast<USCS_Node>(Context);
	
	FSubobjectDataHandle ParentHandle = InParentHandle;
	FSubobjectData* ParentData = ParentHandle.GetData();
	
	check(InSCSNode && ParentData && ParentData->IsValid());
	// Get a handle for us to work with
	FSubobjectDataHandle OutHandle = FactoryCreateSubobjectDataWithParent(InSCSNode, ParentHandle, bIsInherited);
	check(OutHandle.IsValid());
	FSubobjectData* NewData = OutHandle.GetData();
	
	// Determine whether or not the given node is inherited from a parent Blueprint
	USimpleConstructionScript* NodeSCS = InSCSNode->GetSCS();
	
	if( InSCSNode->ComponentTemplate && 
        InSCSNode->ComponentTemplate->IsA(USceneComponent::StaticClass()) && 
        ParentData->IsComponent())
	{
		bool bParentIsEditorOnly = ParentData->GetComponentTemplate()->IsEditorOnly();
		// if you can't nest this new node under the proposed parent (then swap the two)
		if (bParentIsEditorOnly && !InSCSNode->ComponentTemplate->IsEditorOnly() && ParentData->CanReparent())
		{
			FSubobjectData* OldParentPtr = ParentData;
			FSubobjectData* GrandparentPtr = OldParentPtr->GetParentHandle().GetData();

			DetachSubobject(OldParentPtr->GetHandle(), NewData->GetHandle());
			NodeSCS->RemoveNode(OldParentPtr->GetSCSNode());
			const bool bWasRemoved = OutArray.Remove(OldParentPtr->GetHandle()) > 0;
			ensure(bWasRemoved);

			// if the grandparent node is invalid (assuming this means that the parent node was the scene-root)
			if (!GrandparentPtr->IsValid())
			{
				NodeSCS->AddNode(NewData->GetSCSNode());
			}
			else 
			{
				AttachSubobject(GrandparentPtr->GetHandle(), NewData->GetHandle());
			}
			
			// move the proposed parent in as a child to the new node
			AttachSubobject(NewData->GetHandle(), OldParentPtr->GetHandle());

		} // if bParentIsEditorOnly...
	}
	else
	{
		// If the SCS root node array does not already contain the given node, this will add it (this should only occur after node creation)
		if(NodeSCS != nullptr)
		{
			NodeSCS->AddNode(InSCSNode);
		}
	}

	// Recursively add the given SCS node's child nodes
	if(OutHandle.IsValid())
	{
		OutArray.Add(OutHandle);

		// Note: The child array may be modified if a runtime subobject is parented to an editor only subobject
		const TArray<USCS_Node*>& ChildNodes = InSCSNode->GetChildNodes();
		for (int32 i = ChildNodes.Num() - 1; i >= 0; --i)
		{
			USCS_Node* ChildNode = ChildNodes[i];
			FSubobjectDataHandle NewChildHandle = FactoryCreateInheritedBpSubobject(ChildNode, OutHandle, bIsInherited, OutArray);
			ensure(NewChildHandle.IsValid());
			OutArray.Add(NewChildHandle);

			// Since the child array can be modified, we need to ensure that our index remains in bounds
			if (i >= ChildNodes.Num())
			{
				i = ChildNodes.Num();
			}
		}	
	}
	
	return OutHandle;
}

void USubobjectDataSubsystem::RenameSubobjectMemberVariable(UBlueprint* BPContext, const FSubobjectDataHandle& InHandle, const FName NewName)
{
	if (!BPContext || !InHandle.IsValid())
	{
		return;
	}

	if(FSubobjectData* Data = InHandle.GetSharedDataPtr().Get())
	{
		if(USCS_Node* Node = Data->GetSCSNode())
		{
			FBlueprintEditorUtils::RenameComponentMemberVariable(BPContext, Node, NewName);
		}
	}
}

#undef LOCTEXT_NAMESPACE
