// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubobjectDataTedsFactory.h"

#include "SubobjectDataAdapterBase.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace SubobjectDataTedsFactory::Private
{
	FName ReferenceDataHierarchyName = TEXT("SubobjectDataReferencedDataHierarchy");
}

void USubobjectDataTedsFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	SubobjectDataTableHandle = DataStorage.RegisterTable<FSubobjectDataTag, FSubobjectDataSubsystemRowReferenceCollection>(TEXT("SubobjectDataTable"));
	Super::RegisterTables(DataStorage);
}

UE::Editor::DataStorage::TableHandle USubobjectDataTedsFactory::GetSubobjectDataTableHandle() const
{
	return SubobjectDataTableHandle;
}