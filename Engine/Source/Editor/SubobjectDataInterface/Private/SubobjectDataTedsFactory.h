// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "SubobjectDataTedsFactory.generated.h"

UCLASS()
class USubobjectDataTedsFactory final : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~USubobjectDataTedsFactory() override = default;

	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	UE::Editor::DataStorage::TableHandle GetSubobjectDataTableHandle() const;
private:
	UE::Editor::DataStorage::TableHandle SubobjectDataTableHandle = UE::Editor::DataStorage::InvalidTableHandle;
};