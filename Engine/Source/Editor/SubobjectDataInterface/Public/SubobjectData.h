// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "SubobjectDataHandle.h"
#include "Components/ActorComponent.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Engine/SCS_Node.h"		// #TODO_BH  We need to remove this when the actual subobject refactor happens
#include "StructUtils/InstancedStruct.h"

#include "SubobjectData.generated.h"

#define UE_API SUBOBJECTDATAINTERFACE_API

class UBlueprint;
struct FSubobjectDataSubsystemContextDataBase;

/**
* A struct that represents a single subobject. This can be anything, but are
* most commonly components attached to an actor instance or blueprint. Keeps track
* of the handles to its parent object and any child that it has.
*
* If you wish to modify a subobject, use the SubobjectDataSubsystem.
*/
USTRUCT(BlueprintType)
struct FSubobjectData final
{
	friend class USubobjectDataSubsystem;
public:
	GENERATED_BODY()

	UE_API FSubobjectData();

	UE_DEPRECATED(5.7, "Use FSubobjectData default constructor + Initialize function")
	UE_API FSubobjectData(UObject* ContextObject, const FSubobjectDataHandle& ParentHandle, const bool bIsInheritedSCS);

	UE_API void Initialize(UObject* ContextObject, const FSubobjectDataHandle& ParentHandle, const bool bIsInheritedSCS);

	UE_EXPERIMENTAL(5.7, "")
	UE_API void Initialize(TInstancedStruct<FSubobjectDataSubsystemContextDataBase> ContextData, const FSubobjectDataHandle& ParentHandle);
	
	/** True if this subobject's handle is valid */
	bool IsValid() const { return Handle.IsValid(); }

	/**
	* @return	A read-only reference to the object represented by this node.
	*/
	UE_API const UObject* GetObject(bool bEvenIfPendingKill = false) const;

	/**
	* @return A typed struct containing the context object for this SubobjectData
	*         Discrimination types are:
	*           - FSubobjectDataSubsystemContextData_SingleUObjectContextObject
	*           - FSubobjectDataSubsystemContextData_TedsRow
	*/
	UE_EXPERIMENTAL(5.7, "")
	UE_API const TInstancedStruct<FSubobjectDataSubsystemContextDataBase>& GetContextData() const;
	
	/**
	 * @return Whether or not we can edit properties for this subobject
	 */
	UE_API bool CanEdit() const;

	/**
	 * @return Whether or not this object represents a subobject that can be deleted
	 */
	UE_API bool CanDelete() const;

	/**
	* @return Whether or not this object represents a subobject that can be duplicated
	*/
	UE_API bool CanDuplicate() const;

	/**
	* @return Whether or not this object represents a subobject that can be copied
	*/
	UE_API bool CanCopy() const;

	/**
	 * @return Whether or not this object represents a subobject that can 
	 * be reparented to other subobjects based on its context.
	 */
	UE_API bool CanReparent() const;

	/**
	* @return Whether or not this object represents a subobject that can 
	* be renamed.
	*/
	UE_API bool CanRename() const;

	/**
     * @param	InBlueprint			The Blueprint in which the object will be edited.
     *
     * @note	May not be the same as the value returned by GetObject().
     * @return	A reference to the object represented by this node that can be modified within the given Blueprint.
     */
	UE_API const UObject* GetObjectForBlueprint(UBlueprint* Blueprint) const;

	template<class T>
	inline const T* GetObjectForBlueprint(UBlueprint* Blueprint) const
	{
		return Cast<T>(GetObjectForBlueprint(Blueprint));
	}
	
	/**
	 * @param	bEvenIfPendingKill	If false, nullptr will be returned if the cached object instance is pending kill.
	 *								If true, it will be returned regardless (this is used for recaching the object if the objects
	 *								have been reinstanced following construction script execution).
	 *
	 * @note	Deliberately non-virtual, for performance reasons.
	 * @return	A read-only reference to the object represented by this node.
	 */
	template<class T>
	inline const T* GetObject(bool bEvenIfPendingKill = false) const
	{
		return Cast<T>(WeakObjectPtr.Get(bEvenIfPendingKill));
	}

	/**
	 * @param	bEvenIfPendingKill	If false, nullptr will be returned if the cached component template is pending kill.
	 *								If true, it will be returned regardless (this is used for recaching the component template if the objects
	 *								have been reinstanced following construction script execution).
	 *
	 * @note	Deliberately non-virtual, for performance reasons.
	 * @warning This will not return the right component for components overridden by the inherited component handler, you need to call GetOrCreateEditableComponentTemplate instead
	 * @return	The component template or instance represented by this node, if it's a component node.
	 */
	inline const UActorComponent* GetComponentTemplate(bool bEvenIfPendingKill = false) const
	{
		return Cast<UActorComponent>(WeakObjectPtr.Get(bEvenIfPendingKill));
	}

	inline const UActorComponent* FindComponentInstanceInActor(const AActor* InActor) const { return FindMutableComponentInstanceInActor(InActor); }

	UE_API UBlueprint* GetBlueprint() const;
	UE_API UBlueprint* GetBlueprintBeingEdited() const;

	UE_API bool IsInstancedComponent() const;
	UE_API bool IsInstancedActor() const;
	UE_API bool IsNativeComponent() const;
	UE_API bool IsBlueprintInheritedComponent() const;
	UE_API bool IsInheritedComponent() const;
	UE_API bool IsSceneComponent() const;
	UE_API bool IsRootComponent() const;
	UE_API bool IsDefaultSceneRoot() const;
	/** Returns true if the component template's FName starts with USceneComponent::GetDefaultSceneRootVariableName */
	UE_API bool SceneRootHasDefaultName() const;
	/* Returns true if this subobject is a component. */
	UE_API bool IsComponent() const;
	UE_API bool IsChildActor() const;

	/** Returns true if this subobject is a part of a child actor's hierarchy and not the root actor.  */
	UE_API bool IsChildActorSubtreeObject() const;
	UE_API bool IsRootActor() const;
	UE_API bool IsActor() const;
	/** True if this subobject is an instance of an inherited component (its owner is an instanced actor) */
	UE_API bool IsInstancedInheritedComponent() const;

	UE_API bool IsAttachedTo(const FSubobjectDataHandle& InHandle) const;

	UE_API FString GetDisplayString(bool bShowNativeComponentNames = true) const;

	/** Get the display name of this subobject that should be used during drag/drop operations */
	UE_API FText GetDragDropDisplayText() const;

	/**
	 * Returns a string that gives details about this subobject. For Inherited
	 * components it will return "(Inherited)". For native components
	 * this function will return "(NativeComponentName) (Inherited)"
	 */
	UE_API FText GetDisplayNameContextModifiers(bool bShowNativeComponentNames = true) const;
	
	UE_API FText GetDisplayName() const;

	UE_API FName GetVariableName() const;

	// Sockets for attaching in the viewport
	UE_API FText GetSocketName() const;
	UE_API FName GetSocketFName() const;
	UE_API bool HasValidSocket() const;
	UE_API void SetSocketName(FName InNewName);
	UE_API void SetupAttachment(FName SocketName, const FSubobjectDataHandle& AttachParentHandle = FSubobjectDataHandle::InvalidHandle);

	FSubobjectDataHandle GetHandle() const { return Handle; }
	FSubobjectDataHandle GetParentHandle() const { return ParentObjectHandle; }
	FSubobjectData* GetParentData() const { return ParentObjectHandle.GetData(); }
	bool HasParent() const { return ParentObjectHandle.IsValid(); }
	const TArray<FSubobjectDataHandle>& GetChildrenHandles() const { return ChildrenHandles; }
	
	/** Walks the parent hierarchy and returns the root subobject in this case */
	UE_API FSubobjectDataHandle GetRootSubobject() const;

	// Return true if the given handle is a child of this subobject
	UE_API bool HasChild(const FSubobjectDataHandle& ChildHandle) const;

	// Attempt to find the given handle in out child array and return a pointer to it
	UE_API FSubobjectDataHandle FindChild(const FSubobjectDataHandle& ChildHandle) const;
	UE_API FSubobjectDataHandle FindChildByObject(UObject* ContextObject) const;

	/** Get the asset name of this subobject from the asset brokerage */
	UE_API FText GetAssetName() const;

	/** Get the asset path of this subobject from the asset brokerage */
	UE_API FText GetAssetPath() const;

	/** Returns true if the asset this object represents is visible */
	UE_API bool IsAssetVisible() const;

	/**
	* Retrieves tooltip text describing the specified component.
	*
	* @returns An FText object containing a description of the component
	*/
	UE_API FText GetToolTipText() const;

	/**
	* Retrieves tooltip text describing the specified component's mobility.
	*
	* @returns An FText object containing a description of the component's mobility
	*/
	UE_API FText GetMobilityToolTipText() const;

	/**
	* Retrieves a tooltip text describing if the component is marked Editor only or not
	*
	* @returns An FText object containing a description of if the component is marked Editor only or not
	*/
	UE_API FText GetComponentEditorOnlyTooltipText() const;

	/**
	* Retrieves tooltip text describing where the component was first introduced (for inherited components).
	* 
	* @returns An FText object containing a description of when the component was first introduced
	*/
	UE_API FText GetIntroducedInToolTipText() const;

	UE_API FText GetActorDisplayText() const;

	/** 
	 * This type has livliness problems due to cycles in the shared pointer graph. This function clears the pointers 
	 *	involved in cycles, but live FSubobjectDatas will start to behave strangely once it is called
	 */
	UE_API void DisposeUnsafe();

	/**
	 * Removes this FSubobjectData from its parent's children, and clears the parent pointer.
	 */
	UE_EXPERIMENTAL(5.8, "This function may be changed.")
	UE_API void DisconnectFromParent();
private:
	/**
	* Add the given subobject handle as a child of this.
	* NOTE: This does NOT do any actual manipulation of the subobject structure.
	* This only updates this subobject to have the correct handles in it's child structure.
	* Actual manipulation of subobjects should be done via the SubobjectSubsystem.
	*/
	UE_API bool AddChildHandleOnly(const FSubobjectDataHandle& InHandle);

	/**
	* Add the given subobject handle as a child of this.
	* 	* NOTE: This does NOT do any actual manipulation of the subobject structure.
	* This only updates this subobject to have the correct handles in it's child structure.
	* Actual manipulation of subobjects should be done via the SubobjectSubsystem.
	*/
	UE_API bool RemoveChildHandleOnly(const FSubobjectDataHandle& InHandle);

	void SetParentHandle(const FSubobjectDataHandle& NewParentHandle) { ParentObjectHandle = NewParentHandle; }
	void ClearParentHandle() { ParentObjectHandle = FSubobjectDataHandle::InvalidHandle; }
	
	// Weak ptr to the object instance represented by this node (e.g. component template)
	UPROPERTY()
	TWeakObjectPtr<UObject> WeakObjectPtr;

	// Handle that will have a globally unique ID when it is constructed (i.e. when this is constructed)
	UPROPERTY()
	FSubobjectDataHandle Handle;

	// Handle to this object's parent. If this is invalid, then it is the root subobject.
	UPROPERTY()
	FSubobjectDataHandle ParentObjectHandle;

	// Set of child subobjects that use this subobject as a parent
	UPROPERTY()
	TArray<FSubobjectDataHandle> ChildrenHandles;

	// ContextData is a replacement for ContextObject as a mechanism to allow SubobjectData
	// to serve as an interface for multiple data types (not just single UObject) but multiple non-UObject
	// ContextObject will still be populated with a single UObject if available, but ContextData
	// should be the preferred data source for the SubobjectData instances.
	UPROPERTY()
	TInstancedStruct<FSubobjectDataSubsystemContextDataBase> ContextData;

	////////////////////////////////////////////
	// Muteable accessors for this subobject to be used by the subsystem
	inline UObject* GetMutableObject(bool bEvenIfPendingKill = false) const { return WeakObjectPtr.Get(bEvenIfPendingKill); }

	// If this subobject is an actor, then return a pointer to that. Otherwise, search for
	// the owning actor on this component and return it.
	UE_API AActor* GetMutableActorContext();

	template<class T>
    inline T* GetMutableObject(bool bEvenIfPendingKill = false) const
	{
		return Cast<T>(WeakObjectPtr.Get(bEvenIfPendingKill));
	}

	template<class T>
	inline T* GetMutableObjectForBlueprint(UBlueprint* Blueprint) const
    {
    	return const_cast<T*>(GetObjectForBlueprint<T>(Blueprint));
    }
	
	inline UActorComponent* GetMutableComponentTemplate(bool bEvenIfPendingKill = false) const
	{
		return Cast<UActorComponent>(WeakObjectPtr.Get(bEvenIfPendingKill));
	}

	UE_API UActorComponent* FindMutableComponentInstanceInActor(const AActor* InActor) const;

	////////////////////////////////////////////////////
	// Anything related to an SCS node will be changed with an upcoming refactor and should remain private
private:
	// Tries to find a SCS node that was likely responsible for creating the specified instance component.  Note: This is not always possible to do!
	static UE_API USCS_Node* FindSCSNodeForInstance(const UActorComponent* InstanceComponent, UClass* ClassToSearch);

	UE_API USCS_Node* GetSCSNode(bool bEvenIfPendingKill = false) const;

	/**
	* Attempts to set the SCS node member variable based on the WeakObjectPtr.
	* If the weak obj ptr _is_ an USCS_Node type, then we have to change the component
	* template.
	*
	* If this is an instanced component, then we can find the scs node via FindSCSNodeForInstance.
	*
	* @return True if an SCS node has been set
	*/
	UE_API bool AttemptToSetSCSNode();

	// For now, we need a pointer to the SCS node in order to have the correct component template
	// and to be able to get the UBlueprint* from components. This will only be set upon construction
	// if the given Object Context is an USCS_Node. The context object will be set to its template. 
	TWeakObjectPtr<USCS_Node> SCSNodePtr;

	/**
	 * True if this SCS node's blueprint is a child of another Blueprint-generated class,
	 * which means that we must use the InheritableComponentHandler
	 */
	UE_API bool IsInheritedSCSNode() const;

	/** True if this SCS node is inherited from another blueprint generated class. */
	bool bIsInheritedSubobject = false;
	bool bIsInheritedSCS = false;
	bool bIsChildActor = false;
};

#undef UE_API
