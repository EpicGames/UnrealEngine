// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataStorage/Handles.h"
#include "DataStorage/ScopedRowHandle.h"
#include "StructUtils/InstancedStruct.h"

#include "SubobjectDataAdapterBase.generated.h"

class AActor;
class UActorComponent;
class UBlueprint;
class UObject;
struct FSubobjectData;
struct FSubobjectDataHandle;
struct FTedsRowHandle;

namespace UE::Editor
{
	struct FSubobjectEditorContext;
}

#define UE_API SUBOBJECTDATAINTERFACE_API

struct
FSubobjectDataAdapterHandle
{
	friend class USubobjectDataSubsystem;
private:
	uint64 Opaque = 0;
};

struct
FSubobjectDataSubsystemAdapterHandle
{
	friend class USubobjectDataSubsystem;
private:
	uint64 Opaque = 0;
};

USTRUCT()
struct
FSubobjectDataSubsystemContextDataBase
{
	GENERATED_BODY()
};

USTRUCT()
struct FSubobjectDataTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// TEDS column to store a collection of rows that are being displayed under a node in the subobject editor component tree
// Used when SubobjectEditor::UseTedsHierarchies() is false
USTRUCT()
struct FSubobjectDataSubsystemRowReferenceCollection : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FTedsRowHandle> Rows;
};

// An implementation of the ContextData that supports containing a single UObject
// This is to provide interoperability with the existing ContextObject used in APIs of the SubobjectDataSubsystem
USTRUCT()
struct
FSubobjectDataSubsystemContextData_SingleUObjectContextObject : public FSubobjectDataSubsystemContextDataBase
{
	GENERATED_BODY()

	UE_API FSubobjectDataSubsystemContextData_SingleUObjectContextObject();
	UE_API explicit FSubobjectDataSubsystemContextData_SingleUObjectContextObject(UObject* ContextObject);

	UPROPERTY()
	TWeakObjectPtr<UObject> Object;
};

USTRUCT()
struct
FSubobjectDataSubsystemContextData_TedsRow : public FSubobjectDataSubsystemContextDataBase
{
	GENERATED_BODY()

	UE_API FSubobjectDataSubsystemContextData_TedsRow();
	UE_API explicit FSubobjectDataSubsystemContextData_TedsRow(UE::Editor::DataStorage::RowHandle InRowHandle);

	UE::Editor::DataStorage::RowHandle GetRow() const
	{
		return Row ? Row->GetRow() : UE::Editor::DataStorage::InvalidRowHandle;
	}

	UE_API static UE::Editor::DataStorage::TableHandle GetSubobjectItemTable(const UE::Editor::DataStorage::ICoreProvider& DataStorage);
	
private:

	TSharedPtr<UE::Editor::DataStorage::FScopedRow> Row;
};

/**
 * The base of an adapter/plugin that extends the SubobjectDataSubsystem by intercepting
 * calls to the public API.
 * To handle other types of subobjects (even non-UObject based), implement this base class and register it with
 * the SubobjectDataSubsystem instance.
 * Interception is expected to inspect the input data passed to the CanIntercept_* functions and, if they return true, calls the
 * appropriate override.  Multiple adapters follow FIFO priority - first registered is first given a chance to intercept.
 */
class FSubobjectDataSubsystemAdapterBase
{
public:
	UE_API virtual ~FSubobjectDataSubsystemAdapterBase();
	UE_API virtual bool CanIntercept_GatherSubobjectData(const UE::Editor::FSubobjectEditorContext& Context) const;
	UE_API virtual void GatherSubobjectData(const UE::Editor::FSubobjectEditorContext& Context, TArray<FSubobjectDataHandle>& OutArray, TFunctionRef<FSubobjectDataHandle()> CreateSubobjectData) const;

	// Call this implementation of FindSceneRootForSubobject if returning true
	UE_API virtual bool CanIntercept_FindSceneRootForSubobject(const FSubobjectDataHandle& InHandle) const;
	
	// Returns the subobject handle of the "scene root" object - the definition dependent on the implementation.
	// In legacy implementation - returns the actor that owns the provided component ~or~ returns the root SceneComponent of the provided Actor
	UE_API virtual FSubobjectDataHandle FindSceneRootForSubobject(const FSubobjectDataHandle& InHandle) const;
	
	UE_API virtual bool CanIntercept_DeleteSubobjects(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete, UBlueprint* BPContext = nullptr, bool bForce = false) const;
	UE_API virtual int32 DeleteSubobjects(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete) const;

	UE_API virtual bool CanIntercept_CanCopySubobjects(const TArray<FSubobjectDataHandle>& Handles) const;
	UE_API virtual bool CanCopySubobjects(const TArray<FSubobjectDataHandle>& Handles) const;

	UE_API virtual bool CanIntercept_CopySubobjects(const TArray<FSubobjectDataHandle>& Handles, UBlueprint* BpContext) const;
	UE_API virtual void CopySubobjects(const TArray<FSubobjectDataHandle>& Handles, UBlueprint* BpContext) const;

	UE_API virtual bool CanIntercept_CanPasteSubobjects(const FSubobjectDataHandle& RootHandle, UBlueprint* BPContext) const;
	UE_API virtual bool CanPasteSubobjects(const FSubobjectDataHandle& RootHandle,UBlueprint* Blueprint) const;

	UE_API virtual bool CanIntercept_PasteSubobjects(const FSubobjectDataHandle& RootHandle, const TArray<FSubobjectDataHandle>& NewParentHandles, UBlueprint* BPContext) const;
	UE_API virtual void PasteSubobjects(const FSubobjectDataHandle& RootHandle, const TArray<FSubobjectDataHandle>& NewParentHandles, UBlueprint* Blueprint, TArray<FSubobjectDataHandle>& OutPastedHandles) const;
	
	UE_API virtual bool CanIntercept_CanDuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup) const;
	UE_API virtual bool CanDuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup) const;

	UE_API virtual bool CanIntercept_DuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup, UBlueprint* BpContext) const;
	UE_API virtual void DuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup, UBlueprint* BpContext, TArray<FSubobjectDataHandle>& OutNewSubobjects) const;
};


class FSubobjectDataAdapterBase
{
public:
	UE_API virtual ~FSubobjectDataAdapterBase();

	UE_API virtual bool CanEdit(const FSubobjectData& Data) const;
	UE_API virtual bool CanDelete(const FSubobjectData& Data) const;
	UE_API virtual bool CanDuplicate(const FSubobjectData& Data) const;
	UE_API virtual bool CanCopy(const FSubobjectData& Data) const;
	UE_API virtual bool CanReparent(const FSubobjectData& Data) const;
	UE_API virtual bool CanRename(const FSubobjectData& Data) const;

	UE_API virtual const UObject* GetObject(const FSubobjectData& Data, bool bEventIfPendingKill = false) const;
	UE_API virtual const UObject* GetObjectForBlueprint(const FSubobjectData& Data, UBlueprint* Blueprint) const;
	UE_API virtual const UActorComponent* GetComponentTemplate(const FSubobjectData& Data, bool bEvenIfPendingKill = false) const;
	UE_API virtual const UActorComponent* FindComponentInstanceInActor(const FSubobjectData& Data, const AActor* InActor) const;
	UE_API virtual UActorComponent* FindMutableComponentInstanceInActor(const FSubobjectData& Data, const AActor* InActor) const;

	UE_API virtual UBlueprint* GetBlueprint(const FSubobjectData& Data) const;
	UE_API virtual UBlueprint* GetBlueprintBeingEdited(const FSubobjectData& Data) const;

	UE_API virtual bool IsInstancedComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsInstancedActor(const FSubobjectData& Data) const;
	UE_API virtual bool IsNativeComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsBlueprintInheritedComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsInheritedComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsSceneComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsRootComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsDefaultSceneRoot(const FSubobjectData& Data) const;
	UE_API virtual bool SceneRootHasDefaultName(const FSubobjectData& Data) const;
	UE_API virtual bool IsComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsChildActor(const FSubobjectData& Data) const;
	UE_API virtual bool IsChildActorSubtreeObject(const FSubobjectData& Data) const;
	UE_API virtual bool IsRootActor(const FSubobjectData& Data) const;
	UE_API virtual bool IsActor(const FSubobjectData& Data) const;
	UE_API virtual bool IsInstancedInheritedComponent(const FSubobjectData& Data) const;
	UE_API virtual bool IsAttachedTo(const FSubobjectData& Data, const FSubobjectDataHandle& InHandle) const;
	UE_API virtual FString GetDisplayString(const FSubobjectData& Data, bool bShowNativeComponentNames = true) const;
	UE_API virtual FText GetDragDropDisplayText(const FSubobjectData& Data) const;

	UE_API virtual FText GetDisplayNameContextModifiers(const FSubobjectData& Data, bool bShowNativeComponentNames = true) const;
	
	UE_API virtual FText GetDisplayName(const FSubobjectData& Data) const;

	UE_API virtual FName GetVariableName(const FSubobjectData& Data) const;

	// Sockets for attaching in the viewport
	UE_API virtual FText GetSocketName(const FSubobjectData& Data) const;
	UE_API virtual FName GetSocketFName(const FSubobjectData& Data) const;
	UE_API virtual bool HasValidSocket(const FSubobjectData& Data) const;
	UE_API virtual void SetSocketName(FSubobjectData& Data, FName InNewName) const;
	UE_API virtual void SetupAttachment(FSubobjectData& Data, FName SocketName, const FSubobjectDataHandle& AttachParentHandle) const;
	
	UE_API virtual FSubobjectDataHandle FindChildByObject(const FSubobjectData& Data, UObject* ContextObject) const;
	UE_API virtual FText GetAssetName(const FSubobjectData& Data) const;
	UE_API virtual FText GetAssetPath(const FSubobjectData& Data) const;
	UE_API virtual bool IsAssetVisible(const FSubobjectData& Data) const;
	UE_API virtual FText GetToolTipText(const FSubobjectData& Data) const;
	UE_API virtual FText GetMobilityToolTipText(const FSubobjectData& Data) const;
	UE_API virtual FText GetComponentEditorOnlyTooltipText(const FSubobjectData& Data) const;
	UE_API virtual FText GetIntroducedInToolTipText(const FSubobjectData& Data) const;

	UE_API virtual FText GetActorDisplayText(const FSubobjectData& Data) const;
};

#undef UE_API