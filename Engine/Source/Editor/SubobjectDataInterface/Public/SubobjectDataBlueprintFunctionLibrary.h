// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "SubobjectData.h"
#include "SubobjectDataHandle.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SubobjectDataBlueprintFunctionLibrary.generated.h"

#define UE_API SUBOBJECTDATAINTERFACE_API

class UBlueprint;
class UObject;
struct FFrame;

/**
 * A function library with wrappers around the getter/setter functions for FSubobjectData
 * that will make it easier to use within blueprint contexts.
 */
UCLASS(MinimalAPI)
class USubobjectDataBlueprintFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsValid(const FSubobjectData& Data) { return Data.IsValid(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsHandleValid(const FSubobjectDataHandle& DataHandle) { return DataHandle.IsValid(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
    static UE_API void GetData(const FSubobjectDataHandle& DataHandle, FSubobjectData& OutData);
	
	/**
	* @return Get the handle for this subobject data
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Subobject Data")
	static void GetHandle(const FSubobjectData& Data, FSubobjectDataHandle& OutHandle) { OutHandle = Data.GetHandle(); }

	/**
	* @return Get the parent handle for this subobject data
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Subobject Data")
	static void GetParentHandle(const FSubobjectData& Data, FSubobjectDataHandle& OutHandle) { OutHandle = Data.GetParentHandle(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Subobject Data")
	static FText GetDisplayName(const FSubobjectData& Data) { return Data.GetDisplayName(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Subobject Data")
	static FName GetVariableName(const FSubobjectData& Data) { return Data.GetVariableName(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Subobject Data")
	static bool IsAttachedTo(const FSubobjectData& Data, const FSubobjectDataHandle& InHandle) { return Data.IsAttachedTo(InHandle); }

	/**
	* @return Whether or not we can edit properties for this subobject
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool CanEdit(const FSubobjectData& Data) { return Data.CanEdit(); }

	/**
	* @return Whether or not this object represents a subobject that can be deleted
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool CanDelete(const FSubobjectData& Data) { return Data.CanDelete(); }

	/**
	* @return Whether or not this object represents a subobject that can be duplicated
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool CanDuplicate(const FSubobjectData& Data) { return Data.CanDuplicate(); }

	/**
	* @return Whether or not this object represents a subobject that can be copied
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool CanCopy(const FSubobjectData& Data) { return Data.CanCopy(); }

	/**
	* @return Whether or not this object represents a subobject that can 
	* be reparented to other subobjects based on its context.
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool CanReparent(const FSubobjectData& Data) { return Data.CanReparent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool CanRename(const FSubobjectData& Data) { return Data.CanRename(); }
	
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data", meta = (DeprecatedFunction="Note", DeprecationMessage = "Use GetAssociatedObject - it will not return template objects for instances"))
	static UE_API const UObject* GetObject(const FSubobjectData& Data, bool bEvenIfPendingKill = false);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static UE_API const UObject* GetAssociatedObject(const FSubobjectData& Data);

	UFUNCTION(BlueprintCallable, Category="Subobject Data")
	static UE_API const UObject* GetObjectForBlueprint(const FSubobjectData& Data, UBlueprint* Blueprint);

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static UBlueprint* GetBlueprint(const FSubobjectData& Data) { return Data.GetBlueprint(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsInstancedComponent(const FSubobjectData& Data) { return Data.IsInstancedComponent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsInstancedActor(const FSubobjectData& Data) { return Data.IsInstancedActor(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsNativeComponent(const FSubobjectData& Data) { return Data.IsNativeComponent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsInheritedComponent(const FSubobjectData& Data) { return Data.IsInheritedComponent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsSceneComponent(const FSubobjectData& Data) { return Data.IsSceneComponent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsRootComponent(const FSubobjectData& Data) { return Data.IsRootComponent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsDefaultSceneRoot(const FSubobjectData& Data) { return Data.IsDefaultSceneRoot(); }

	/* Returns true if this subobject is a component. */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsComponent(const FSubobjectData& Data) { return Data.IsComponent(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsChildActor(const FSubobjectData& Data) { return Data.IsChildActor(); }

	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsRootActor(const FSubobjectData& Data) { return Data.IsRootActor(); }
	
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Subobject Data")
	static bool IsActor(const FSubobjectData& Data) { return Data.IsActor(); }
};

#undef UE_API
