// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "SubobjectDataHandle.generated.h"

#define UE_API SUBOBJECTDATAINTERFACE_API

struct FSubobjectData;

/**
* A subobject handle is a globally unique identifier for subobjects
* Upon construction, the handle will be invalid. It is the responsibility
* of the owning FSubobjectData to set the DataPtr once the subobject
* data has validated that it has a good context.
*/
USTRUCT(BlueprintType)
struct FSubobjectDataHandle
{
	friend struct FSubobjectData;
	friend class USubobjectDataSubsystem;
	
	GENERATED_USTRUCT_BODY();

	explicit FSubobjectDataHandle()
		: DataPtr(nullptr)
	{}
	FSubobjectDataHandle(const TSharedPtr<FSubobjectData> Data)
		: DataPtr(Data)
	{}
	
	~FSubobjectDataHandle()
	{
		DataPtr = nullptr;
	}

	/**
	* True if the Handle is valid (i.e. not INDEX_NONE). This is true once GenerateNewHandle is called
	*
	* @return bool		True if this handle is valid
	*/
	inline bool IsValid() const { return DataPtr != nullptr; }

	bool operator==(const FSubobjectDataHandle& Other) const
	{
		return DataPtr.Get() == Other.DataPtr.Get();
	}

	bool operator!=(const FSubobjectDataHandle& Other) const
	{
		return DataPtr != Other.DataPtr;
	}

	/** Returns a pointer to the subobject data that this is a handle for */
	inline TSharedPtr<FSubobjectData> GetSharedDataPtr() const { return DataPtr; }
	inline FSubobjectData* GetData() const { return DataPtr.IsValid() ? DataPtr.Get() : nullptr; }

	/**
	* A static representation of an invalid handle. Mostly used as a return
	* value for functions that need to return a handle but have the risk of being invalid.
	*/
	static UE_API const FSubobjectDataHandle InvalidHandle;
	
	/** Get the hash code to use for the given FSubobjectDataHandle */
	friend uint32 GetTypeHash(const FSubobjectDataHandle& Key)
	{
		return PointerHash(Key.DataPtr.Get());
	}
	
private:
	/** Pointer to the actual subobject data that this handle represents */
	TSharedPtr<FSubobjectData> DataPtr;
};

template<>
struct TStructOpsTypeTraits<FSubobjectDataHandle> : public TStructOpsTypeTraitsBase2<FSubobjectDataHandle>
{
	enum
	{
		WithIdenticalViaEquality = true,
	};
};

#undef UE_API
