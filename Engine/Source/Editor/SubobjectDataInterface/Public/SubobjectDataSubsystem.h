// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Subsystems/EngineSubsystem.h"
#include "SubobjectData.h"
#include "ModuleDescriptor.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "SubobjectDataAdapterBase.h"

#include "SubobjectDataSubsystem.generated.h"

#define UE_API SUBOBJECTDATAINTERFACE_API

class UActorComponent;
class FScopedTransaction;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewSubobjectAdded, const FSubobjectData&);

/** Options when adding a new subobject */
USTRUCT(BlueprintType, Category = "SubobjectDataSubsystem")
struct FAddNewSubobjectParams
{
	GENERATED_USTRUCT_BODY()

	FAddNewSubobjectParams()
		: ParentHandle(FSubobjectDataHandle::InvalidHandle)
		, NewClass(nullptr)
		, AssetOverride(nullptr)
		, BlueprintContext(nullptr)
		, bSkipMarkBlueprintModified(false)
		, bConformTransformToParent(true)
	{
	}
	
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	FSubobjectDataHandle ParentHandle;

	/** The class of the new subobject that will be added */
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	TObjectPtr<UClass> NewClass;

	/** Specific asset to use instead of the selected asset in the content browser */
	UObject* AssetOverride;

	/**
	 * Pointer to the blueprint context that this subobject is in. If this is null, it is assumed that
	 * this subobject is being added to an instance.
	 */
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	TObjectPtr<UBlueprint> BlueprintContext;
	
	/** Optionally skip marking this blueprint as modified (e.g. if we're handling that externally */
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	uint8 bSkipMarkBlueprintModified : 1;

	/** Whether the newly created component should keep its transform, or conform it to its parent */
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	uint8 bConformTransformToParent : 1;
};

/** Options for reparenting subobjects */
USTRUCT(BlueprintType, Category = "SubobjectDataSubsystem")
struct FReparentSubobjectParams
{
	GENERATED_USTRUCT_BODY()

	FReparentSubobjectParams() = default;

	/** The handle of the subobject to reparent to. */
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	FSubobjectDataHandle NewParentHandle;

	/**
	* Pointer to the blueprint context that this subobject is in. If this is null, it is assumed that
	* this subobject is being added to an instance.
	*/
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	TObjectPtr<UBlueprint> BlueprintContext = nullptr;

	/** 
	* The preview actor context to be used if in a blueprint context. 
	* This must have a value if BlueprintContext is needed. 
	*/
	UPROPERTY(BlueprintReadWrite, Category = "SubobjectDataSubsystem")
	TObjectPtr<AActor> ActorPreviewContext = nullptr;
};

namespace UE::Editor
{
	//~ It would be nice if this were a nested class inside USubobjectDataSubsystem, but
	//~  we need it in SubobjectDataAdapterBase.h, we can't forward declare nested classes,
	//~  and it's a pain to forward declare everything from that file here instead.
	/**
	 * Struct holding information about the context of a subobject editor
	 */
	struct FSubobjectEditorContext
	{
		// If editor operates on a single object, the object to use. I.e. the object
		//  typically returned by GetObjectContext inside a details panel. This might
		//  eventually be phased out in favor of always using SelectionSetName or
		//  some other way to identify the target objects (since there may be multiple)
		UObject* ObjectContext = nullptr;
		
		TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet = nullptr;
	};
}

/**
* The Subobject Data Subsystem will produce the reflected subobject data
* based on a given root object. A root object can be anything, an actor 
* instance clicked on via the level editor, a UBlueprint* by opening an asset,
* or something piped in from python or other scripting languages. 
*/
UCLASS(MinimalAPI, Category = "SubobjectDataSubsystem")
class USubobjectDataSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()
public:

	/** Implement this for initialization of instances of the system */
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	/** Implement this for deinitialization of instances of the system */
	UE_API virtual void Deinitialize() override;
	
	/**
	* Static wrapper for getting this engine subsystem. Will return nullptr
	* if the SubobjectDataInterface module has not been loaded yet.
	*/
	static UE_API USubobjectDataSubsystem* Get();
	
	/**
	* Gather all subobjects that the given UObject context has. Populates an array of 
	* handles that will have the given context and all it's subobjects.
	* 
	* @param ObjectContext	Object to gather subobjects for
	* @param OutArray		Array to populate (will be emptied first)
	*/
	UE_API void GatherSubobjectData(UObject* ObjectContext, TArray<FSubobjectDataHandle>& OutArray);

	/**
	* Gather all subobjects that the given context has. Populates an array of
	* handles that will have the relevant subobjects (taken from selection, or from
	* an object, as appropriate).
	*
	* @param OutArray		Array to populate (will be emptied first)
	*/
	UE_EXPERIMENTAL(5.8, "Use GatherSubobjectData(UObject*, ...) for now.  This API may change.")
	UE_API void GatherSubobjectData(const UE::Editor::FSubobjectEditorContext& Context, TArray<FSubobjectDataHandle>& OutArray);


	////////////////////////////////////////////
	// System events
	
	/** Delegate invoked when a new subobject is successfully added */
	FOnNewSubobjectAdded& OnNewSubobjectAdded() { return OnNewSubobjectAdded_Delegate; };

	using FSubobjectDataAdapterFilter = TFunction<bool(const FSubobjectData&)>;
	/** */
	UE_EXPERIMENTAL(5.7, "Experimental adapter to adjust behavior of SubobjectData")
	UE_API FSubobjectDataAdapterHandle RegisterSubobjectDataAdapter(FSubobjectDataAdapterFilter InFilter, TUniquePtr<FSubobjectDataAdapterBase> InDataAdapter);
	UE_EXPERIMENTAL(5.7, "Experimental adapter to adjust behavior of SubobjectData")
	UE_API void UnregisterSubobjectDataAdapter(FSubobjectDataAdapterHandle Handle);
	UE_EXPERIMENTAL(5.7, "Experimental adapter to adjust behavior of SubobjectData")
	const FSubobjectDataAdapterBase* FindDataAdapter(const FSubobjectData& SubobjectData) const;

	using FSubobjectSubsystemAdapterFilter = TFunction<bool(const TInstancedStruct<FSubobjectDataSubsystemContextDataBase>& ContextData)>;
	/** */
	UE_EXPERIMENTAL(5.7, "Experimental adapter to adjust behavior of SubobjectDataSubsystem")
	UE_API FSubobjectDataSubsystemAdapterHandle RegisterSubsystemAdapter(TUniquePtr<FSubobjectDataSubsystemAdapterBase> InAdapter);
	UE_EXPERIMENTAL(5.7, "Experimental adapter to adjust behavior of SubobjectDataSubsystem")
	UE_API void UnregisterSubsystemAdapter(FSubobjectDataSubsystemAdapterHandle Handle);

	////////////////////////////////////////////

	/**
	* Gather all subobjects that the given Blueprint context has. Populates an array of
	* handles that will have the given context and all it's subobjects.
	*
	* @param Context		Object to gather subobjects for
	* @param OutArray		Array to populate (will be emptied first)
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Gather Subobject Data For Blueprint"))
	UE_API void K2_GatherSubobjectDataForBlueprint(UBlueprint* Context, TArray<FSubobjectDataHandle>& OutArray);

	/**
	* Gather all subobjects that the given actor instance has. Populates an array of
	* handles that will have the given context and all it's subobjects.
	*
	* @param Context		Object to gather subobjects for
	* @param OutArray		Array to populate (will be emptied first)
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Gather Subobject Data For Instance"))
	UE_API void K2_GatherSubobjectDataForInstance(AActor* Context, TArray<FSubobjectDataHandle>& OutArray);

	/**
	* Attempt to find the subobject data for a given handle. OutData will only 
	* be valid if the function returns true.
	*
	* @param Handle		Handle of the subobject data you want to acquire
	* @param OutData	Reference to the subobject data to populate
	*
	* @return bool		true if the data was found
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "FindSubobjectDataFromHandle"))
	UE_API bool K2_FindSubobjectDataFromHandle(const FSubobjectDataHandle& Handle, FSubobjectData& OutData) const;

	/**
	* Attempt to find an existing handle for the given object. 
	*
	* @param Context		The context that the object to find is within
	* @param ObjectToFind	The object that you want to find the handle for within the context
	*
	* @return FSubobjectDataHandle	The subobject handle for the object, Invalid handle if not found.
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API FSubobjectDataHandle FindHandleForObject(const FSubobjectDataHandle& Context, const UObject* ObjectToFind, UBlueprint* BPContext = nullptr) const;
	
	////////////////////////////////////////////
	// Modifying subobjects

	/**
	 * Creates a new C++ component from the specified class type
	 * The user will be prompted to pick a new subclass name and code will be recompiled
	 *
	 * @return The new class that was created
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Create New C++ Component"))
	static UE_API UClass* CreateNewCPPComponent(TSubclassOf<UActorComponent> ComponentClass, const FString& NewClassPath, const FString& NewClassName);

	/**
	 * Creates a new Blueprint component from the specified class type
	 * The user will be prompted to pick a new subclass name and a blueprint asset will be created
	 *
	 * @return The new class that was created
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Create New Blueprint Component"))
	static UE_API UClass* CreateNewBPComponent(TSubclassOf<UActorComponent> ComponentClass, const FString& NewClassPath, const FString& NewClassName);
	
	/**
	* Add a new subobject as a child to the given parent object 
	*
	* @param Params			Options to consider when adding this subobject
	*
	* @return FSubobjectDataHandle		Handle to the newly created subobject, Invalid handle if creation failed
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API FSubobjectDataHandle AddNewSubobject(const FAddNewSubobjectParams& Params, FText& FailReason);

	/**
	* Attempts to delete the given array of subobjects from their context 
	*
	* @param ContextHandle			The owning context of the subobjects that should be removed
	* @param SubobjectsToDelete		Array of subobject handles that should be deleted
	* @param BPContext				The blueprint context for the given
	* 
	* @return 	The number of subobjects successfully deleted
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Delete Subobjects from Blueprint"))
	UE_API int32 DeleteSubobjects(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete, UBlueprint* BPContext = nullptr);

	/**
	* Attempts to delete the given array of subobjects from their context 
	*
	* @param ContextHandle			The owning context of the subobjects that should be removed
	* @param SubobjectsToDelete		Array of subobject handles that should be deleted
	*
	* @return 	The number of subobjects successfully deleted
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Delete Subobjects from Instance"))
	UE_API int32 K2_DeleteSubobjectsFromInstance(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete);
	
	/**
	* Attempts to delete the given array of subobjects from their context 
	*
	* @param ContextHandle			The owning context of the subobjects that should be removed
	* @param SubobjectsToDelete		Array of subobject handles that should be deleted
	* @param OutComponentToSelect	Populates this handle with a valid selection in the component hierarchy if desired
	* @param bForce					If true then this will attempt to delete a subobject even if the CanDelete function flags otherwise.
	* @return 	The number of subobjects successfully deleted
	*/
	UE_API int32 DeleteSubobjects(const FSubobjectDataHandle& ContextHandle, const TArray<FSubobjectDataHandle>& SubobjectsToDelete, FSubobjectDataHandle& OutComponentToSelect, UBlueprint* BPContext = nullptr, bool bForce = false);

	/**
	* Attempts to delete the given subobject from its blueprint context 
	*
	* @param ContextHandle			The owning context of the subobjects that should be removed
	* @param SubobjectToDelete		The subobject handles that should be deleted
	* @param BPContext				The blueprint context for the given 
	*
	* @return 	The number of subobjects successfully deleted
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Delete Subobject from Blueprint"))
	UE_API int32 DeleteSubobject(const FSubobjectDataHandle& ContextHandle, const FSubobjectDataHandle& SubobjectToDelete, UBlueprint* BPContext = nullptr);
	
	/**
	* Attempts to delete the given subobject from its context 
	*
	* @param ContextHandle			The owning context of the subobjects that should be removed
	* @param SubobjectToDelete		The subobject handles that should be deleted
	*
	* @return 	The number of subobjects successfully deleted
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem", meta = (DisplayName = "Delete Subobject from Instance"))
	UE_API int32 K2_DeleteSubobjectFromInstance(const FSubobjectDataHandle& ContextHandle, const FSubobjectDataHandle& SubobjectToDelete);
	
	/**
	* Attempts to rename the given subobject to the new name.
	*
	* @param Handle			Handle to the subobject to rename
	* @param InNewName		The new name that is desired for the given subobject
	*
	* @return True if the rename was successful, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool RenameSubobject(const FSubobjectDataHandle& Handle, const FText& InNewName);

	/**
	* Attempts to change the subclass of a native component
	*
	* @param Handle			Handle to the subobject to change class of
	* @param NewClass		The new class that is desired for the given subobject
	*
	* @return True if the class change was successful, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool ChangeSubobjectClass(const FSubobjectDataHandle& Handle, const UClass* NewClass);

	/**
	* Attempts to reparent the given subobject to the new parent
	*
	* @param NewParentHandle		Handle of the new parent
	* @param ToReparentHandle		The handle of the subobject that will get moved
	*
	* @return True if the reparent was successful, false otherwise.
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
    UE_API bool ReparentSubobject(const FReparentSubobjectParams& Params, const FSubobjectDataHandle& ToReparentHandle);

	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool MakeNewSceneRoot(const FSubobjectDataHandle& Context, const FSubobjectDataHandle& NewSceneRoot, UBlueprint* BPContext);
	
	/**
	* Attempts to reparent all subobjects in the HandlesToMove array to the new parent handle.
	* 
	* @param NewParentHandle		Handle of the new parent
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool ReparentSubobjects(const FReparentSubobjectParams& Params, const TArray<FSubobjectDataHandle>& HandlesToMove);

	/**
	 * Remove the child subobject from the owner
	 *
	 * @return True if the child was successfully removed. 
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool DetachSubobject(const FSubobjectDataHandle& OwnerHandle, const FSubobjectDataHandle& ChildToRemove);

	/**
	 * Add the given subobject to a new owner. This will remove the subobject from its previous 
	 * owner if necessary.
	 *
	 * @param OwnerHandle		The new owner to attach to
	 * @param ChildToAddHandle	Handle to the subobject that will become a child of the owner
	 * 
	 * @return true if the child was added successfully
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
    UE_API bool AttachSubobject(const FSubobjectDataHandle& OwnerHandle, const FSubobjectDataHandle& ChildToAddHandle);
	
	/**
	 * Returns true if the given new text is a valid option to rename the
	 * subobject with the given handle. Populates the OutErrorMessage if
	 * it is not valid.
	 *
	 * @param Handle			Handle to the subobject that is being checked
	 * @param InNewText			The new name that is desired
	 * @param OutErrorMessage	The reasoning for an invalid name
	 *
	 * @return True if the rename is valid
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool IsValidRename(const FSubobjectDataHandle& Handle, const FText& InNewText, FText& OutErrorMessage) const;

	/**
	 * Returns true if the given array of handles represents subobjects that
	 * can be copied.
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool CanCopySubobjects(const TArray<FSubobjectDataHandle>& Handles) const;

	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API void CopySubobjects(const TArray<FSubobjectDataHandle>& Handles, UBlueprint* BpContext);

	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API bool CanPasteSubobjects(const FSubobjectDataHandle& RootHandle, UBlueprint* BPContext = nullptr) const;

	/**
	* Pastes the given subobjects to the PasteToContext.
	* 
	* @param PasteToContext		The subobject to paste things onto
	* @param NewParentHandles	Array of Subobject Handles that you would like to paste
	* @param BpContext			Blueprint to use, if you are pasting to a blueprint. Null if pasting to an instanced object
	* @param OutPastedHandles	Array populated with the handles to the newly pasted subobjects
	*/
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API void PasteSubobjects(const FSubobjectDataHandle& PasteToContext, const TArray<FSubobjectDataHandle>& NewParentHandles, UBlueprint* BpContext, TArray<FSubobjectDataHandle>& OutPastedHandles);

	UE_API bool CanDuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup);
	
	/**
	 * Duplicate the given array of subobjects on the context. 
	 *
	 * @param Context			The owning context that the subobjects to dup come from
	 * @param SubobjectsToDup	Array of handles of existing subobjects you would like to have duplicated
	 * @param BpContext			Pointer to the current blueprint context if necessary. Use nullptr if dealing with instances
	 * @param OutNewSubobjects	Array that will be populated with any newly created subobjects
	 */
	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	UE_API void DuplicateSubobjects(const FSubobjectDataHandle& Context, const TArray<FSubobjectDataHandle>& SubobjectsToDup, UBlueprint* BpContext, TArray<FSubobjectDataHandle>& OutNewSubobjects);
	
	UE_API FScopedTransaction* BeginTransaction(const TArray<FSubobjectDataHandle>& Handles, const FText& Description, UBlueprint* Blueprint);

	/**
	* Find the scene root for a given subobject handle.
	* If this is an actor, then it will return the handle pointing to its DefaultSceneRoot.
	* If this is a component, it will walk the parent chain until it finds the DefaultSceneRoot.
	*/
	UE_API FSubobjectDataHandle FindSceneRootForSubobject(const FSubobjectDataHandle& InHandle) const;

	UFUNCTION(BlueprintCallable, Category = "SubobjectDataSubsystem")
	static UE_API void RenameSubobjectMemberVariable(UBlueprint* BPContext, const FSubobjectDataHandle& InHandle, const FName NewName);
	
	/**
	* Walk the hierarchy up to find the AActor root for the given handle
	* 
	* @param StartingHandle		The handle to start walking the hierarchy at
	*
	* @return The handle of the root actor if one exists, FSubobjectDataHandle::InvalidHandle otherwise.
	*/
	static UE_API FSubobjectDataHandle GetActorRootHandle(const FSubobjectDataHandle& StartingHandle);

private:

	/**
	* Recursively add the attachment children of the given actor component
	*/
	UE_API FSubobjectDataHandle FindOrCreateAttachParentForComponent(UActorComponent* InActorComponent, const FSubobjectDataHandle& ActorRootHandle, TArray<FSubobjectDataHandle>& ExistingHandles);

	/**
	* Recursively search for an attachment parent that matches the given InActorComponent on the blueprint
	*/
	UE_API FSubobjectDataHandle FindAttachParentForInheritedComponent(UActorComponent* InActorComponent, const FSubobjectDataHandle& ActorRootHandle, UBlueprint* BP);

	/**
	 * Find all child subobject data related to the given root context.
	 * Populates the OutVisited set with all data. 
	 */
	UE_API void FindAllSubobjectData(FSubobjectData* RootContext, TSet<FSubobjectData*>& OutVisited) const;
	
	/**
	 * Creates an invalid SubobjectData and associates a handle
	 * The SubobjectData can be initialized using the Initialize() function
	 */
	UE_API FSubobjectDataHandle CreateSubobjectData();
	/**
	 * Gets a ref to subobject data that has been initialized with the given context.
	 * It will have a unique ID, but may be reused from the FreeData array. 
	 */
	UE_API FSubobjectDataHandle CreateSubobjectData(UObject* Context, const FSubobjectDataHandle& ParentHandle = FSubobjectDataHandle::InvalidHandle, bool bIsInheritedSCS = false);

	/**
	 * Get a handle to the subobject with the given context with a specific parent.
	 * If the context is already a known child of the parent, then it will return an existing subobject handle. 
	 */
	UE_API FSubobjectDataHandle FactoryCreateSubobjectDataWithParent(UObject* Context, const FSubobjectDataHandle& ParentHandle, bool bIsInheritedSCS = false);

	/**
	 * Create a subobject data and handle for an inherited blueprint context.
	 * This will populate the given array of handles recursively with all possible children
	 * (which are USCS_Node* in this case)
	 */
	UE_API FSubobjectDataHandle FactoryCreateInheritedBpSubobject(UObject* Context, const FSubobjectDataHandle& ParentHandle, bool bIsInherited, TArray<FSubobjectDataHandle>& OutArray);
	
	/** Find the parent that this new subobject should attach to */
	UE_API FSubobjectDataHandle FindParentForNewSubobject(const UObject* NewSubobject, const FSubobjectDataHandle& SelectedParent);

	FOnNewSubobjectAdded OnNewSubobjectAdded_Delegate;

	struct FSubobjectDataAdapterEntry
	{
		TUniquePtr<FSubobjectDataAdapterBase> Adapter;
		FSubobjectDataAdapterHandle Handle;
		FSubobjectDataAdapterFilter Filter;
	};
	uint64 DataAdapterHandleIndex = 0;
	TArray<FSubobjectDataAdapterEntry> DataAdapters;

	struct FSubsystemAdapterEntry
	{
		TUniquePtr<FSubobjectDataSubsystemAdapterBase> Adapter;
		FSubobjectDataSubsystemAdapterHandle Handle;
	};
	uint64 SubsystemAdapterHandleIndex = 0;
	TArray<FSubsystemAdapterEntry> SubsystemAdapters;
	
};

#undef UE_API
