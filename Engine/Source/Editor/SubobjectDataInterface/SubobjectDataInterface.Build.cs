// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class SubobjectDataInterface : ModuleRules
{
	public SubobjectDataInterface(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"TypedElementFramework"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BlueprintGraph",
				"Engine",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"UnrealEd",	// ComponentEditorUtils, AssetBrokerage
					"GameProjectGeneration",
					"TypedElementFramework"
				}
			);
		}
	}
}
